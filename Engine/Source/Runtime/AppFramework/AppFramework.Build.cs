// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AppFramework : ModuleRules
{
	public AppFramework(ReadOnlyTargetRules Target) : base(Target)
	{
		NumIncludedBytesPerUnityCPPOverride = 196608; // best unity size found from using UBT ProfileUnitySizes mode

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
                "CoreUObject",
				"ApplicationCore",
				"Slate",
				"SlateCore",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"InputCore",
			}
		);

		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"SlateReflector",
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"SlateReflector",
				}
			);
		}

		bDisableAutoRTFMInstrumentation = true;
	}
}
