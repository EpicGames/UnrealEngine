// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/NameTypes.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


static const FName AppFrameworkTabName("AppFramework");


/**
 * Implements the AppFramework module.
 */
class FAppFrameworkModule
	: public IModuleInterface
{
public:

	// IModuleInterface interface
	
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};


IMPLEMENT_MODULE(FAppFrameworkModule, AppFramework);
