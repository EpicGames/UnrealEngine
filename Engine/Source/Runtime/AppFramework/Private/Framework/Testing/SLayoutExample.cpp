// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Testing/SLayoutExample.h"
#include "Misc/Paths.h"
#include "Widgets/SNullWidget.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/CoreStyle.h"
#include "Application/SlateWindowHelper.h"
#include "SlateOptMacros.h"
#include "Widgets/SWeakWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"

#if !UE_BUILD_SHIPPING


#define LOCTEXT_NAMESPACE "ExampleLayoutTest"


class SExampleLayout
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SExampleLayout)
	{ }
	SLATE_END_ARGS()
	
	/**
	 * Construct the widget
	 *
	 * @param InArgs   Declaration from which to construct the widget.
	 */
	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	void Construct(const FArguments& InDelcaration)
	{
		const FVector2D HeadingShadowOffset(2,2);

		TSharedPtr<int32> SharedSizeValue = MakeShared<int32>(1);
		TAttribute<float> SizeAttrib = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateLambda([SharedSizeValue] { return (float)*SharedSizeValue.Get(); })); 
		
		FSlateFontInfo LargeLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 16);
		FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);
		this->ChildSlot
		[
			SNew(SScrollBox)
			// Default settings example
			+SScrollBox::Slot() .Padding(5)
			[
				SNew(STextBlock) .ShadowOffset(HeadingShadowOffset) .Font( LargeLayoutFont ) .Text( LOCTEXT("ExampleLayout-DefaultSettingsLabel", "Default Settings (AutoSize):") )
			]
			+SScrollBox::Slot() .Padding(10,5)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton) .Text( LOCTEXT("ExampleLayout-TextLabel01", "Default.\n Slot is auto-sized.") )
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton) .Text( LOCTEXT("ExampleLayout-TextLabel02", "Slots are packed tightly.") )
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton) .Text( LOCTEXT("ExampleLayout-TextLabel03", "Alignment within the slot\n does not matter.") )
				]
			]
			// Fill Size example
			+SScrollBox::Slot() .Padding(5)
			[
				SNew(STextBlock) .ShadowOffset(HeadingShadowOffset) .Font( LargeLayoutFont ) .Text( LOCTEXT("ExampleLayout-FillSizeLabel", "Fill Size:") )
			]
			+SScrollBox::Slot() .Padding(10,5)
			[
				SNew(STextBlock) .Font(SmallLayoutFont) .Text( LOCTEXT("ExampleLayout-TextLabel04", "Will stretch to fill any available room based on the fill coefficients.") )
			]
			+SScrollBox::Slot() .Padding(10,5)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot() .FillWidth(2)
				[
					SNew(SButton) .Text( LOCTEXT("ExampleLayout-TextLabel05", ".FillWidth(2)") )
				]
				+SHorizontalBox::Slot() .FillWidth(1)
				[
					SNew(SButton) .Text( LOCTEXT("ExampleLayout-TextLabel06", ".FillWidth(1)") )
				]
				+SHorizontalBox::Slot() .FillWidth(3)
				[
					SNew(SButton) .Text( LOCTEXT("ExampleLayout-TextLabel07", ".FillWidth(3)") )
				]
			]
			// Fill Contents example
			+SScrollBox::Slot() .Padding(5.f)
			[
				SNew(STextBlock) .ShadowOffset(HeadingShadowOffset) .Font( LargeLayoutFont ) .Text( LOCTEXT("ExampleLayout-FillContentsLabel", "Fill Content Size:") )
			]
			+SScrollBox::Slot() .Padding(10.f,5.f)
			[
				SNew(STextBlock) .Font(SmallLayoutFont) .Text( LOCTEXT("ExampleLayout-FillContent0", "Calculated size tarts with content size (like auto), but allows to control shrinking and growing.") )
			]
			+SScrollBox::Slot() .Padding(10.f,5.f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot() .FillContentWidth(0.f, 1.f)
				[
					SNew(SButton) .Text( LOCTEXT("ExampleLayout-FillContent1", "No grow, allow shrink.\n.FillContentWidth(0, 1)") )
				]
				+SHorizontalBox::Slot() .FillContentWidth(SizeAttrib)
				[
					SNew(SButton)
					.Text_Lambda([SharedSizeValue]()
					{
						return FText::Format(LOCTEXT("ExampleLayout-FillContent2", "Allow grow and Shrink\n.FillContentWidth({0})"), *SharedSizeValue.Get());
					})
					.OnClicked_Lambda([SharedSizeValue]()
					{
						int32& Value = *SharedSizeValue;
						Value = (Value + 1) % 4;
						return FReply::Handled();						
					})
				]
				+SHorizontalBox::Slot() .FillContentWidth(1.f, 0.f)
				[
					SNew(SButton) .Text( LOCTEXT("ExampleLayout-FillContent3", "Allow grow, no shrink\n.FillContentWidth(1, 0)") )
				]
			]
			// Aspect Ratio example
			+SScrollBox::Slot() .Padding(5)
			[
				SNew(STextBlock) .ShadowOffset(HeadingShadowOffset) .Font( LargeLayoutFont ) .Text( LOCTEXT("ExampleLayout-AspectRatiolabel", "Aspect Ratio:") )
			]
			+SScrollBox::Slot() .Padding(5)
			.HAlign(HAlign_Left)
			[
				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding( 2.0f, 1.0f, 4.0f, 1.0f )
				[
					SNew( SSpacer )
					.Size( FVector2D(16.0f, 16.0f) )
				]

				// Label text
				+ SHorizontalBox::Slot()
				.FillWidth( 1.0f )
				.Padding( 2.0f, 1.0f, 2.0f, 1.0f )
				[
					SNew( STextBlock )
					.Text( LOCTEXT("ExampleLayout-TextLabel08", "Somewhat lengthy text. Apricot." ) )
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding( 2.0f, 1.0f, 2.0f, 1.0f )
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew( SImage )
					.Image( FCoreStyle::Get().GetBrush( "Menu.SubMenuIndicator" ) )
				]
			]
			+SScrollBox::Slot() .Padding(5)
			[
				SNew(STextBlock) .Text( LOCTEXT("ExampleLayout-TextLabel09", "A somewhat long piece of text.") )
			]
			// Fixed Size example
			+SScrollBox::Slot() .Padding(5)
			[
				SNew(STextBlock) .ShadowOffset(HeadingShadowOffset) .Font( LargeLayoutFont ) .Text( LOCTEXT("ExampleLayout-AlignmentLabel", "Alignment:") )
			]
			+SScrollBox::Slot() .Padding(5)
			[
				SNew(STextBlock)
				.Font(SmallLayoutFont)
				.WrapTextAt(400.0f)
				.Text( LOCTEXT("ExampleLayout-TextLabel10", "SBox supports various alignments, padding and a fixed override for the content's desired size. FixedSize is rarely needed. If your content appears too large, never crush it by forcing a fixed size. Instead, figure out why the content's DesiredSize is too large! Making extra room via FixedSize is not as bad.") )
			]
			+SScrollBox::Slot()
			[
				// Align example
				SNew(SVerticalBox)
				// Top-Aligned Fixed Size
				+SVerticalBox::Slot() .FillHeight(1)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSpacer)
						.Size(FVector2D(0.0f, 128.0f))
					]
					+SHorizontalBox::Slot() .FillWidth(1)
					[
						SNew(SButton)
						.ToolTipText( LOCTEXT("ExampleLayout-TopLeftTextLabel", "Top Left") )
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Top)
						[
							SNew(SBorder) .Content()[ SNew(STextBlock) .Text( LOCTEXT("ExampleLayout-TopLeftTextLabel", "Top Left") ) ]
						]
					]
					+SHorizontalBox::Slot() .FillWidth(1)
					[
						SNew(SButton)
						.ToolTipText( LOCTEXT("ExampleLayout-TopCenterTextLabel", "Top Center") )
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Top)
						[
							SNew(SBorder) .Content()[ SNew(STextBlock) .Text( LOCTEXT("ExampleLayout-TopCenterTextLabel", "Top Center") ) ]
						]
					]
					+SHorizontalBox::Slot() .FillWidth(1)
					[
						SNew(SButton)
						.ToolTipText( LOCTEXT("ExampleLayout-TopRightTextLabel", "Top Right") )
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Top)
						[
							SNew(SBorder) .Content()[ SNew(STextBlock) .Text( LOCTEXT("ExampleLayout-TopRightTextLabel", "Top Right") ) ]
						]
					]
					+SHorizontalBox::Slot() .FillWidth(1)
					[
						SNew(SButton)
						.ToolTipText( LOCTEXT("ExampleLayout-TopFillTextLabel", "Top Fill") )
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Top)
						[
							SNew(SBorder) .Content()[ SNew(STextBlock) .Text( LOCTEXT("ExampleLayout-TopFillTextLabel", "Top Fill") ) ]
						]
					]
				]
				// Center-Aligned Fixed Size
				+SVerticalBox::Slot() .FillHeight(1)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSpacer)
						.Size(FVector2D(0.0f, 128.0f))
					]
					+SHorizontalBox::Slot() .FillWidth(1)
					[
						SNew(SButton)
						.ToolTipText( LOCTEXT("ExampleLayout-CenterLeftTextLabel", "Center Left") )
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						[
							SNew(SBorder) .Content()[ SNew(STextBlock) .Text( LOCTEXT("ExampleLayout-CenterLeftTextLabel", "Center Left") ) ]
						]
					]
					+SHorizontalBox::Slot() .FillWidth(1)
					[
						SNew(SButton)
						.ToolTipText( LOCTEXT("ExampleLayout-CenterCenterTextLabel", "Center Center") )
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(SBorder) .Content()[ SNew(STextBlock) .Text( LOCTEXT("ExampleLayout-CenterCenterTextLabel", "Center Center") ) ]
						]
					]
					+SHorizontalBox::Slot() .FillWidth(1)
					[
						SNew(SButton)
						.ToolTipText( LOCTEXT("ExampleLayout-CenterRightTextLabel", "Center Right") )
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)
						[
							SNew(SBorder) .Content()[ SNew(STextBlock) .Text( LOCTEXT("ExampleLayout-CenterRightTextLabel", "Center Right") ) ]
						]
					]
					+SHorizontalBox::Slot() .FillWidth(1)
					[
						SNew(SButton)
						.ToolTipText( LOCTEXT("ExampleLayout-CenterFillTextLabel", "Center Fill") )
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Center)
						[
							SNew(SBorder) .Content()[ SNew(STextBlock) .Text( LOCTEXT("ExampleLayout-CenterFillTextLabel", "Center Fill") ) ]
						]
					]
				]
				// Bottom-Aligned Fixed Size
				+SVerticalBox::Slot() .FillHeight(1)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSpacer)
						.Size(FVector2D(0.0f, 128.0f))
					]
					+SHorizontalBox::Slot()  .FillWidth(1)
					[
						SNew(SButton)
						.ToolTipText( LOCTEXT("ExampleLayout-BottomLeftTextLabel", "Bottom Left") )
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Bottom)
						[
							SNew(SBorder) .Content()[ SNew(STextBlock) .Text( LOCTEXT("ExampleLayout-BottomLeftTextLabel", "Bottom Left") ) ]
						]
					]
					+SHorizontalBox::Slot()  .FillWidth(1)
					[
						SNew(SButton)
						.ToolTipText( LOCTEXT("ExampleLayout-BottomCenterTextLabel", "Bottom Center") )
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Bottom)
						[
							SNew(SBorder) .Content()[ SNew(STextBlock) .Text( LOCTEXT("ExampleLayout-BottomCenterTextLabel", "Bottom Center") ) ]
						]
					]
					+SHorizontalBox::Slot()  .FillWidth(1)
					[
						SNew(SButton)
						.ToolTipText( LOCTEXT("ExampleLayout-BottomRightTextLabel", "Bottom Right") )
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Bottom)
						[
							SNew(SBorder) .Content()[ SNew(STextBlock) .Text( LOCTEXT("ExampleLayout-BottomRightTextLabel", "Bottom Right") ) ]
						]
					]
					+SHorizontalBox::Slot()  .FillWidth(1)
					[
						SNew(SButton)
						.ToolTipText( LOCTEXT("ExampleLayout-BottomFillTextLabel", "Bottom Fill") )
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Bottom)
						[
							SNew(SBorder) .Content()[ SNew(STextBlock) .Text( LOCTEXT("ExampleLayout-BottomFillTextLabel", "Bottom Fill") ) ]
						]
					]
				]
				// Fill-Aligned Fixed Size
				+SVerticalBox::Slot() .FillHeight(1)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSpacer)
						.Size(FVector2D(0.0f, 128.0f))
					]
					+SHorizontalBox::Slot() .FillWidth(1)
					[
						SNew(SButton)
						.ToolTipText( LOCTEXT("ExampleLayout-FillLeftTextLabel", "Fill Left") )
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Fill)
						[
							SNew(SBorder) .Content()[ SNew(STextBlock) .Text( LOCTEXT("ExampleLayout-FillLeftTextLabel", "Fill Left") ) ]
						]
					]
					+SHorizontalBox::Slot() .FillWidth(1)
					[
						SNew(SButton)
						.ToolTipText( LOCTEXT("ExampleLayout-FillCenterTextLabel", "Fill Center") )
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Fill)
						[
							SNew(SBorder) .Content()[ SNew(STextBlock) .Text( LOCTEXT("ExampleLayout-FillCenterTextLabel", "Fill Center") ) ]
						]
					]
					+SHorizontalBox::Slot() .FillWidth(1)
					[
						SNew(SButton)
						.ToolTipText( LOCTEXT("ExampleLayout-FillRightTextLabel", "Fill Right") )
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Fill)
						[
							SNew(SBorder) .Content()[ SNew(STextBlock) .Text( LOCTEXT("ExampleLayout-FillRightTextLabel", "Fill Right") ) ]
						]
					]
					+SHorizontalBox::Slot() .FillWidth(1)
					[
						SNew(SButton)
						.ToolTipText( LOCTEXT("ExampleLayout-FillFillTextLabel", "Fill Fill") )
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						[
							SNew(SBorder) .Content()[ SNew(STextBlock) .Text( LOCTEXT("ExampleLayout-FillFillTextLabel", "Fill Fill") ) ]
						]
					]
				] // End Align Example
			]
			+SScrollBox::Slot()
			[
				SAssignNew(TooltipArea, SBorder)				

			]
		];
	}

	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

	
	virtual bool OnVisualizeTooltip( const TSharedPtr<SWidget>& TooltipContent ) override
	{
		// The layout example has its own way of visualizing tool tips.
		// We show them below the items.
		if (TooltipContent.IsValid())
		{
			TooltipArea->SetContent
			(
				// It is crucial to present the tooltip content using an SWeakWidget
				// because we are merely showing the content; the hovered widget is
				// the tooltip owner.
				SNew(SWeakWidget).PossiblyNullContent( TooltipContent )
			);
		}
		else
		{
			TooltipArea->SetContent( SNullWidget::NullWidget );
		}

		return true;
	}

	TSharedPtr<SBorder> TooltipArea;

};




TSharedRef<SWidget> MakeLayoutExample()
{
	extern TOptional<FSlateRenderTransform> GetTestRenderTransform();
	extern FVector2D GetTestRenderTransformPivot();
	return
		SNew(SExampleLayout)
		.RenderTransform_Static(&GetTestRenderTransform)
		.RenderTransformPivot_Static(&GetTestRenderTransformPivot);
}

#undef LOCTEXT_NAMESPACE

#endif // #if !UE_BUILD_SHIPPING
