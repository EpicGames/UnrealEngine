// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Testing/SStarshipGallery.h"

#include "HAL/FileManager.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#include "Animation/CurveSequence.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"

#include "Types/SlateEnums.h"
#include "UObject/ReflectedTypeAccessors.h"
// #include "Types/SlateEnums.h"

// #include "SlateCore.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Containers/ObservableArray.h"

#include "Layout/WidgetPath.h"
#include "SlateOptMacros.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Testing/SWidgetGallery.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SFxWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SSpinningImage.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SHeader.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SUniformWrapPanel.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STileView.h"
#include "Widgets/Views/STreeView.h"


#include "Brushes/SlateImageBrush.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#if !UE_BUILD_SHIPPING


#define LOCTEXT_NAMESPACE "StarshipGallery"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

/**
 * Implements a widget gallery to develop the UE5 Slate 
 *
 * The widget gallery demonstrates the widgets available in the core of the Slate.
 * Update the PopulateGallery() method to add your new widgets.
 */


class SStarshipGallery
    : public SCompoundWidget


{

public:

    SLATE_BEGIN_ARGS( SStarshipGallery ) { }
    SLATE_END_ARGS()
        
public:

    /**
     * Constructs the widget gallery.
     *
     * @param InArgs - Construction arguments.
     */


    void Construct( const FArguments& InArgs )
    {

        TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("StarshipStyleGallery")
        ->AddArea
        (
            FTabManager::NewPrimaryArea()
            ->Split
            (
                FTabManager::NewStack()
                ->AddTab("Colors", ETabState::OpenedTab)
                ->AddTab("Text", ETabState::OpenedTab)
                ->AddTab("Icons", ETabState::OpenedTab)
                ->AddTab("Starship Widgets", ETabState::OpenedTab)
                ->AddTab("List Widgets", ETabState::OpenedTab)
                // ->AddTab("SLATE WIDGETS", ETabState::OpenedTab)
                ->SetForegroundTab(FName("Starship Widgets"))
            )
        );

        FGlobalTabmanager::Get()->RegisterNomadTabSpawner("Colors", 
            FOnSpawnTab::CreateLambda( 
                [this](const FSpawnTabArgs&) -> TSharedRef<SDockTab>
                {
                    return SNew(SDockTab)
                    .TabRole(ETabRole::PanelTab)
                    .ContentPadding(0)
                    [
                        ConstructColorsGallery()
                    ];
                }
            )
        );

        FGlobalTabmanager::Get()->RegisterNomadTabSpawner("Text", 
            FOnSpawnTab::CreateLambda( 
                [this](const FSpawnTabArgs&) -> TSharedRef<SDockTab>
                {
                    return SNew(SDockTab)
                    .TabRole(ETabRole::PanelTab)
                    .ContentPadding(0)
                     [
                        ConstructTextGallery()
                    ];
                }
            )
        );


        FGlobalTabmanager::Get()->RegisterNomadTabSpawner("Icons", 
            FOnSpawnTab::CreateLambda( 
                [this](const FSpawnTabArgs&) -> TSharedRef<SDockTab>
                {
                    return SNew(SDockTab)
                    .TabRole(ETabRole::PanelTab)
                    .ContentPadding(0)
                     [
                        ConstructIconsGallery()
                    ];
                }
            )
        );


        FGlobalTabmanager::Get()->RegisterNomadTabSpawner("Starship Widgets", 
            FOnSpawnTab::CreateLambda( 
                [this](const FSpawnTabArgs&) -> TSharedRef<SDockTab>
                {
                    return SNew(SDockTab)
                    .TabRole(ETabRole::PanelTab)
                    .ContentPadding(0) 
                    .ForegroundColor(FSlateColor::UseStyle())
                    [
                        ConstructWidgetGallery()
                    ];
                }
            )
        );

		FGlobalTabmanager::Get()->RegisterNomadTabSpawner("List Widgets",
			FOnSpawnTab::CreateLambda(
				[this](const FSpawnTabArgs&) -> TSharedRef<SDockTab>
				{
					return SNew(SDockTab)
						.TabRole(ETabRole::PanelTab)
						.ContentPadding(0)
						.ForegroundColor(FSlateColor::UseStyle())
						[
							ConstructListGallery()
						];
				}
			)
		);


        /*
        FGlobalTabmanager::Get()->RegisterTabSpawner("SLATE WIDGETS", 
            FOnSpawnTab::CreateLambda( 
                [this](const FSpawnTabArgs&) -> TSharedRef<SDockTab>
                {
                    return SNew(SDockTab) .TabRole(ETabRole::NomadTab) [
                        MakeWidgetGallery()
                    ];
                }
            )
        );*/


        // FOnSpawnTab::CreateLambda(SpawnTableViewTesting));
        ChildSlot
        [
            FGlobalTabmanager::Get()->RestoreFrom(Layout, TSharedPtr<SWindow>() ).ToSharedRef()
        ];
    };

    TSharedRef<SWidget> ConstructColorsGallery()
    {

        auto GenerateColorButton = [] (FText InTitle, FName StyleColorName, bool Inverted = false, bool Outline = false) -> TSharedRef<SWidget> 
        {

            FSlateColor InSlateColor = FAppStyle::Get().GetSlateColor( StyleColorName );
            FSlateColor OutlineColor = FAppStyle::Get().GetSlateColor("Colors.Foldout");
            // TSharedRef<SButton> ColorButton;
            // = SNew(SButton)
            return SNew(SBox)
            .Padding(8)
            .WidthOverride(120)
            .HeightOverride(100)
            [
                    SNew(SOverlay)

                    +SOverlay::Slot()
                    [
                        SNew(SBorder)
                        .BorderImage(Outline ? new FSlateRoundedBoxBrush(InSlateColor, 6.f, OutlineColor, 1.0f) : new FSlateRoundedBoxBrush(InSlateColor, 6.0f))
                        // .Color(InSlateColor.GetSpecifiedColor())
                    ]

                    +SOverlay::Slot()
                    .Padding(12)
                    [
                        SNew(SVerticalBox)
                        +SVerticalBox::Slot()
                        .FillHeight(1)
                        .VAlign(VAlign_Bottom)
                        [
                            SNew(STextBlock)
                            .Font(FAppStyle::Get().GetFontStyle("SmallFontBold"))
                            .ColorAndOpacity( FAppStyle::Get().GetSlateColor( Inverted ? "Colors.Background": "Colors.White" ) )
                            .Text(InTitle)
                        ]

                        +SVerticalBox::Slot()
                        .VAlign(VAlign_Bottom)
                        .AutoHeight()
                        [
                            SNew(STextBlock)
                            .Font(FAppStyle::Get().GetFontStyle("SmallFont"))
                            .ColorAndOpacity( FAppStyle::Get().GetSlateColor( Inverted ? "Colors.Background" : "Colors.White" ) )
                            .Text( FText::FromString( InSlateColor.GetSpecifiedColor().ToFColor(true).ToHex() ) )
                        ]
                    ]
            ];
        };

        FSlateColor LabelColor = FAppStyle::Get().GetSlateColor("Colors.White50");

        return SNew(SBorder)
        [

         SNew(SScrollBox)
        + SScrollBox::Slot()
        .Padding(48)
        [
        

            SNew(SVerticalBox)

            +SVerticalBox::Slot()
            .AutoHeight()
            .Padding(8.f, 24.f, 8.f, 8.f)
            [
                SNew(STextBlock).ColorAndOpacity(LabelColor).Text(NSLOCTEXT("StarshipGallery", "BaseColors", "BASE COLORS"))
            ]

            +SVerticalBox::Slot()
            .AutoHeight()
            [
                SNew(SUniformWrapPanel)
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("BLACK"),           "Colors.Black")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("TITLE"),           "Colors.Title")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("WINDOW\nBORDER"),  "Colors.WindowBorder")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("FOLDOUT"),         "Colors.Foldout")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("INPUT"),           "Colors.Input")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("RECESSED"),        "Colors.Recessed")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("BACKGROUND"),      "Colors.Background", false, true)]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("HEADER"),          "Colors.Header")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("DROPDOWN"),        "Colors.Dropdown")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("HOVER"),           "Colors.Hover")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("HOVER2"),          "Colors.Hover2")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("WHITE"),           "Colors.White", true)]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("WHITE25"),         "Colors.White25")]

                /*
            ]

            +SVerticalBox::Slot()
            .AutoHeight()
            [
                SNew(SUniformWrapPanel)
*/
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("PRIMARY"),         "Colors.Primary", true)]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("PRIMARY\nHOVER"),  "Colors.PrimaryHover", true)]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("PRIMARY\nPRESS"),  "Colors.PrimaryPress", true)]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("SECONDARY"),       "Colors.Secondary", true)]
            ]

            +SVerticalBox::Slot()
            .Padding(8.f, 24.f, 8.f, 8.f)
            .AutoHeight()
            [
                SNew(STextBlock).ColorAndOpacity(LabelColor).Text(NSLOCTEXT("StarshipGallery", "TextIconColors", "FOREGROUND COLORS"))

            ]

            +SVerticalBox::Slot()
            .AutoHeight()
            [
                SNew(SUniformWrapPanel)

                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("FOREGROUND"),        "Colors.Foreground")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("FOREGROUND\nHOVER"), "Colors.ForegroundHover", true)]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("FOREGROUND\nINVERTED"), "Colors.ForegroundInverted")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("HIGHLIGHT"),         "Colors.Highlight")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("FOREGROUND\nHEADER"), "Colors.ForegroundHeader", true)]
                /*
            ]

            +SVerticalBox::Slot()
            .AutoHeight()
            [
                SNew(SUniformWrapPanel)
            */
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("SELECT"),          "Colors.Select", true)]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("SELECT\nHOVER"),   "Colors.SelectHover")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("SELECT\nPARENT"),  "Colors.SelectParent")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("SELECT\nINACTIVE"),"Colors.SelectInactive")]
            ]

            +SVerticalBox::Slot()
            .Padding(8.f, 24.f, 8.f, 8.f)
            .AutoHeight()
            [
                SNew(STextBlock).ColorAndOpacity(LabelColor).Text(NSLOCTEXT("StarshipGallery", "AccentColors", "ACCENT COLORS"))
            ]

            +SVerticalBox::Slot()
            .AutoHeight()
            [
                SNew(SUniformWrapPanel)
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("BLUE"),      "Colors.AccentBlue", true)]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("PURPLE"),    "Colors.AccentPurple")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("PINK"),      "Colors.AccentPink", true)]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("RED"),       "Colors.AccentRed", true)]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("ORANGE"),    "Colors.AccentOrange", true)]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("YELLOW"),    "Colors.AccentYellow", true)]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("GREEN"),     "Colors.AccentGreen", true)]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("BROWN"),     "Colors.AccentBrown")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("FOLDER"),    "Colors.AccentFolder")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("BLACK"),     "Colors.AccentBlack", false, true)]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("GRAY"),      "Colors.AccentGray")]
                +SUniformWrapPanel::Slot()[ GenerateColorButton( FText::FromString("WHITE"),     "Colors.AccentWhite", true)]
            ]
        ]];


    };

    TSharedPtr<SSearchBox> IconSearchBox;
    bool IsIconVisible(const FString& IconPath) const
    {
        if (!IconSearchBox.IsValid() || IconSearchBox->GetText().IsEmpty())
        {
            return true;
        }
        return IconPath.Contains(IconSearchBox->GetText().ToString());
    }

    TArray< TUniquePtr< FSlateBrush > > DynamicBrushes;
    TSharedRef<SWidget> ConstructIconsGallery()
    {
        // auto GenerateColorButton = [] (FText InTitle, FName StyleColorName, bool Inverted = false) -> TSharedRef<SWidget> 

        auto GenerateIconLibrary = [this] (FText InTitle, FString InPath) -> TSharedRef<SWidget>
        {
            const FVector2D IconSize(20.f, 20.f);
            TSharedPtr<SUniformWrapPanel> UniformWrapPanel = SNew(SUniformWrapPanel)
            .HAlign(HAlign_Left)
            .SlotPadding( FMargin(12.f, 12.f) );

            TArray<FString> FoundIcons;
            FString SearchDirectory = FPaths::EngineDir() /  InPath;// TEXT("Editor/Slate/Icons/GeneralTools");
            // IFileManager::Get().FindFiles(FoundIcons, *SearchDirectory, TEXT(".png"));//, true, true, false);
            IFileManager::Get().FindFilesRecursive(FoundIcons, *SearchDirectory, TEXT("*.png"), true, false);
            for (const FString& Filename : FoundIcons)
            {
                // FString IconPath = SearchDirectory / Filename;
                FString IconPath = Filename;

                DynamicBrushes.Add( TUniquePtr<FSlateDynamicImageBrush>(new FSlateDynamicImageBrush( FName(*IconPath), IconSize )));

                UniformWrapPanel->AddSlot()
                [
                    SNew(SImage)
                    .Image(DynamicBrushes.Last().Get())
                    .ToolTipText( FText::FromString( IconPath ) )
                     .Visibility_Lambda([this, IconPath]() { return IsIconVisible(IconPath) ? EVisibility::Visible : EVisibility::Collapsed; })
                ];
            }

            return SNew(SVerticalBox)
            +SVerticalBox::Slot()
            .AutoHeight()
            [
                SNew(STextBlock).Text(InTitle)
            ]

            +SVerticalBox::Slot(        )
            [
                UniformWrapPanel.ToSharedRef()
            ];
        };

        auto GenerateIconLibrarySVG = [this] (FText InTitle, FString InPath) -> TSharedRef<SWidget>
        {
            const FVector2D IconSize(20.f, 20.f);
            TSharedPtr<SUniformWrapPanel> UniformWrapPanel = SNew(SUniformWrapPanel)
            .HAlign(HAlign_Left)
            .SlotPadding( FMargin(12.f, 12.f) );

            TArray<FString> FoundIcons;
            FString SearchDirectory = FPaths::EngineDir() /  InPath;// TEXT("Editor/Slate/Icons/GeneralTools");
            // IFileManager::Get().FindFiles(FoundIcons, *SearchDirectory, TEXT(".png"));//, true, true, false);
            IFileManager::Get().FindFilesRecursive(FoundIcons, *SearchDirectory, TEXT("*.svg"), true, false);
            for (const FString& Filename : FoundIcons)
            {
                // FString IconPath = SearchDirectory / Filename;
                FString IconPath = Filename;

                DynamicBrushes.Add( TUniquePtr<FSlateVectorImageBrush>(new FSlateVectorImageBrush( IconPath, IconSize )));

                UniformWrapPanel->AddSlot()
                [
                    SNew(SImage)
                    .Image(DynamicBrushes.Last().Get())
                    .ToolTipText( FText::FromString( IconPath ) )
                    .Visibility_Lambda([this, IconPath]() { return IsIconVisible(IconPath) ? EVisibility::Visible : EVisibility::Collapsed; })
                ];
            }

            return SNew(SVerticalBox)
            +SVerticalBox::Slot()
            .AutoHeight()
            [
                SNew(STextBlock).Text(InTitle)
            ]

            +SVerticalBox::Slot(        )
            [
                UniformWrapPanel.ToSharedRef()
            ];
        };



        return SNew(SBorder)
        .BorderImage( FAppStyle::Get().GetBrush("ToolPanel.GroupBorder") )
        [
            SNew(SScrollBox)
            + SScrollBox::Slot()
            .Padding(48)
            [
                SNew(SVerticalBox)

                + SVerticalBox::Slot()
                .AutoHeight()
                .Padding(0.0f, 0.0f, 0.0f, 4.0f)
                [
                    SAssignNew(IconSearchBox, SSearchBox)
                    .HintText(LOCTEXT("IconSearchHint", "Enter text to filter icons by path..."))
                ]

                +SVerticalBox::Slot().AutoHeight()[ GenerateIconLibrarySVG(NSLOCTEXT("StarshipGallery", "SlateCore", "Core"), "Content/Slate/Starship/Common")]
                +SVerticalBox::Slot().AutoHeight()[ GenerateIconLibrarySVG(NSLOCTEXT("StarshipGallery", "Editor Common", "Editor"), "Content/Editor/Slate/Starship/Common")]

                +SVerticalBox::Slot().AutoHeight()[ GenerateIconLibrarySVG(NSLOCTEXT("StarshipGallery", "SceneOutliner", "SceneOutliner"), "Content/Editor/Slate/Starship/SceneOutliner")]
                +SVerticalBox::Slot().AutoHeight()[ GenerateIconLibrarySVG(NSLOCTEXT("StarshipGallery", "GraphEditors", "GraphEditors"), "Content/Editor/Slate/Starship/GraphEditors")]
                // +SVerticalBox::Slot().AutoHeight()[ GenerateIconLibrarySVG(NSLOCTEXT("StarshipGallery", "LevelEditor", "LevelEditor"), "Content/Editor/Slate/Starship/LevelEditor/Menus")]
                +SVerticalBox::Slot().AutoHeight()[ GenerateIconLibrarySVG(NSLOCTEXT("StarshipGallery", "MainToolbar", "MainToolbar"), "Content/Editor/Slate/Starship/MainToolbar")]
                // +SVerticalBox::Slot().AutoHeight()[ GenerateIconLibrarySVG(NSLOCTEXT("StarshipGallery", "FileMenu", "FileMenu"), "Content/Editor/Slate/Starship/Menus/File")]
                // +SVerticalBox::Slot().AutoHeight()[ GenerateIconLibrarySVG(NSLOCTEXT("StarshipGallery", "EditMenu", "EditMenu"), "Content/Editor/Slate/Starship/Menus/Edit")]
                // +SVerticalBox::Slot().AutoHeight()[ GenerateIconLibrarySVG(NSLOCTEXT("StarshipGallery", "HelpMenu", "HelpMenu"), "Content/Editor/Slate/Starship/Menus/Help")]

                +SVerticalBox::Slot().AutoHeight()[ GenerateIconLibrary(NSLOCTEXT("StarshipGallery", "PaintIconTitle", "Paint"), "Content/Editor/Slate/Icons/Paint")]
                +SVerticalBox::Slot().AutoHeight()[ GenerateIconLibrary(NSLOCTEXT("StarshipGallery", "LandscapeIconTitle", "Landscape"), "Content/Editor/Slate/Icons/Landscape")]
                +SVerticalBox::Slot().AutoHeight()[ GenerateIconLibrary(NSLOCTEXT("StarshipGallery", "ModelingIconTitle", "Modeling"), "/Plugins/Editor/ModelingToolsEditorMode/Content/Icons")]
                +SVerticalBox::Slot().AutoHeight()[ GenerateIconLibrary(NSLOCTEXT("StarshipGallery", "FractureIconTitle", "Fracture"), "/Plugins/Experimental/ChaosEditor/Content")]
                +SVerticalBox::Slot().AutoHeight()[ GenerateIconLibrary(NSLOCTEXT("StarshipGallery", "CurveEditorIconTitle", "CurveEditor"), "Content/Editor/Slate/GenericCurveEditor/Icons")]
                +SVerticalBox::Slot().AutoHeight()[ GenerateIconLibrary(NSLOCTEXT("StarshipGallery", "GeneralIconTitle", "General"), "Content/Editor/Slate/Icons/GeneralTools")]
				+SVerticalBox::Slot().AutoHeight()[ GenerateIconLibrarySVG(NSLOCTEXT("StarshipGallery", "TimelineEditorIconTitle", "TimelineEditor"), "Content/Editor/Slate/Starship/TimelineEditor")]
            ]
        ];
    }

    TSharedRef<SWidget> ConstructTextGallery()
    {

        FSlateColor LabelColor = FAppStyle::Get().GetSlateColor("Colors.White50");

        return SNew(SBorder)
        .Padding(48.f)
        [
            SNew(SHorizontalBox)

            +SHorizontalBox::Slot()
            .AutoWidth()
            .Padding(64.f, 0.0f)
            [

                SNew(SVerticalBox)

                // Normal 
                +SVerticalBox::Slot().Padding(0.f, 64.f, 0.f, 4.f).AutoHeight() [ SNew(STextBlock).ColorAndOpacity(LabelColor).Text( NSLOCTEXT("StarshipGallery", "NormalBodyTextDesc", "ROBOTO 10 SLATE    [ ROBOTO 13 FIGMA ]") ) ]
                +SVerticalBox::Slot().Padding(0.f, 4.f).AutoHeight() [ 
                    SNew(STextBlock)
                    .Text(NSLOCTEXT("StarshipGallery", "NormalBodyText", "Normal Text\n\nThe quick brown fox jumps over the lazy dog.\n\nNORMAL TEXT CAPITAL") ) 
                ]

                // Small 
                +SVerticalBox::Slot().Padding(0.f, 64.f, 0.f, 4.f).AutoHeight()[ SNew(STextBlock).ColorAndOpacity(LabelColor).Text(NSLOCTEXT("StarshipGallery", "SmallBodyTextDesc", "ROBOTO 8 SLATE   [ ROBOTO 11 FIGMA ]" ) ) ]
                +SVerticalBox::Slot().Padding(0.f, 4.f).AutoHeight()[ 
                    SNew(STextBlock)
                    .Font(FAppStyle::Get().GetFontStyle("SmallFont"))
                    .Text(NSLOCTEXT("StarshipGallery", "SmallBodyText", "Small Text\n\nThe quick brown fox jumps over the lazy dog.\n\nSMALL TEXT CAPITAL") ) 
                ]
            ]

            /*
            // Extra Large
            +SVerticalBox::Slot().Padding(0.f, 32.f, 0.f, 4.f).AutoHeight()[ SNew(STextBlock).ColorAndOpacity(LabelColor).Text(NSLOCTEXT("StarshipGallery", "ExtraLargeBodyTextDesc", "ROBOTO 14") ) ]
            +SVerticalBox::Slot().Padding(0.f, 4.f).AutoHeight()[ 
                SNew(STextBlock)
                .Font(FAppStyle::Get().GetFontStyle("ExtraLargeFont"))
                .Text(NSLOCTEXT("StarshipGallery", "ExtraLargeBodyText", "Extra Large Body Text\n\nThe quick brown fox jumps over the lazy dog.\n\nEXTRA LARGE TEXT CAPITAL") ) 
            ]

            // Large
            +SVerticalBox::Slot().Padding(0.f, 32.f, 0.f, 4.f).AutoHeight()[ SNew(STextBlock).ColorAndOpacity(LabelColor).Text(NSLOCTEXT("StarshipGallery", "LargeBodyTextDesc", "ROBOTO 11") ) ]
            +SVerticalBox::Slot().Padding(0.f, 4.f).AutoHeight()[ 
                SNew(STextBlock)
                .Font(FAppStyle::Get().GetFontStyle("LargeFont"))
                .Text(NSLOCTEXT("StarshipGallery", "LargeBodyText", "Large Text\n\nThe quick brown fox jumps over the lazy dog.\n\nLARGE TEXT CAPITAL") ) 
            ]
            */


            +SHorizontalBox::Slot()
            .AutoWidth()
            .Padding(64.f, 0.0f)
            [
                SNew(SVerticalBox)

                // Normal Bold
                +SVerticalBox::Slot().Padding(0.f, 64.f, 0.f, 4.f).AutoHeight() [ SNew(STextBlock).ColorAndOpacity(LabelColor).Text( NSLOCTEXT("StarshipGallery", "NormalBoldTextDesc", "ROBOTO BOLD 10 SLATE   [ ROBOTO 13 FIGMA ]") ) ]
                +SVerticalBox::Slot().Padding(0.f, 4.f).AutoHeight() [ 
                    SNew(STextBlock)
                    .Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
                    .Text(NSLOCTEXT("StarshipGallery", "NormalBodyBoldText", "Normal Text Bold \n\nThe quick brown fox jumps over the lazy dog.\n\nNORMAL TEXT BOLD CAPITAL") ) 
                ]

                // Small Bold
                +SVerticalBox::Slot().Padding(0.f, 64.f, 0.f, 4.f).AutoHeight()[ SNew(STextBlock).ColorAndOpacity(LabelColor).Text(NSLOCTEXT("StarshipGallery", "SmallBoldTextDesc", "ROBOTO BOLD 8 SLATE   [ ROBOTO 11 FIGMA ]") ) ]
                +SVerticalBox::Slot().Padding(0.f, 4.f).AutoHeight()[ 
                    SNew(STextBlock)
                    .Font(FAppStyle::Get().GetFontStyle("SmallFontBold"))
                    .Text(NSLOCTEXT("StarshipGallery", "SmallBodyBoldText", "Small Text Bold \n\nThe quick brown fox jumps over the lazy dog.\n\nSMALL TEXT BOLD CAPITAL") ) 
                ]
            ]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(64.f, 0.0f)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot().Padding(0.f, 64.f, 0.f, 4.f).AutoHeight()[SNew(STextBlock).ColorAndOpacity(LabelColor).Text(NSLOCTEXT("StarshipGallery", "SubMathTextDesc", "SUB-TYPEFACES - MATH"))]
				+ SVerticalBox::Slot().Padding(0.f, 4.f).AutoHeight()
				[
					SNew(STextBlock)
					.Text(NSLOCTEXT("StarshipGallery", "MathematicalTextSpecimen", "Mathematical Alphanumeric Symbols, U+1D400 - U+1D7FF\n\U0001D400\U0001D401\U0001D402 \U0001D434\U0001D435\U0001D436 \U0001D4D0\U0001D4D1\U0001D4D2"))
				]

				+ SVerticalBox::Slot().Padding(0.f, 64.f, 0.f, 4.f).AutoHeight()[SNew(STextBlock).ColorAndOpacity(LabelColor).Text(NSLOCTEXT("StarshipGallery", "SubEmojiTextDesc", "SUB-TYPEFACES - EMOJI"))]
				+ SVerticalBox::Slot().Padding(0.f, 4.f).AutoHeight()
				[
					SNew(STextBlock)
					.Text(NSLOCTEXT("StarshipGallery", "EmoticonsTextSpecimen", "Emoticons, U+1F600 - U+1F64F\n\U0001F60E\U0001F643\U0001F648"))
				]
				+ SVerticalBox::Slot().Padding(0.f, 4.f).AutoHeight()
				[
					SNew(STextBlock)
					.Text(NSLOCTEXT("StarshipGallery", "SupplementalSymbolsPictographsTextSpecimen", "Supplemental Symbols and Pictographs, U+1F900 - U+1F9FF\n\U0001F914\U0001F916\U0001F9F2"))
				]
			]
        ];
    }

    // State Variables 
    int RadioChoice;
    int SegmentedBoxChoice;
    TArray< TSharedPtr< FString > > ComboItems;

    float NumericEntryBoxChoice;

	void MakeMenuEntries(FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.AddMenuEntry(LOCTEXT("MenuEntry1", "Menu Entry 1"), FText::GetEmpty(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.box-perspective"), FUIAction());
		MenuBuilder.AddMenuEntry(LOCTEXT("MenuEntry2", "Menu Entry 2"), FText::GetEmpty(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.box-perspective"), FUIAction());
		MenuBuilder.AddMenuEntry(LOCTEXT("MenuEntry3", "Menu Entry 3"), FText::GetEmpty(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.box-perspective"), FUIAction());
		MenuBuilder.AddMenuEntry(LOCTEXT("MenuEntry4", "Menu Entry 4"), FText::GetEmpty(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.box-perspective"), FUIAction());
	};

	TSharedRef<SWidget> BuildToolbar(bool bIncludeSettingsButtons)
	{
		FSlimHorizontalToolBarBuilder Toolbar(nullptr, FMultiBoxCustomization::None);

		FUIAction ButtonAction;
		Toolbar.AddToolBarButton(ButtonAction, NAME_None, LOCTEXT("ToolBarButton1", "Button"), FText::GetEmpty(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.box-perspective"));

		static bool IsChecked = false;
		FUIAction ToggleButtonAction;
		ToggleButtonAction.GetActionCheckState.BindLambda([&] {return IsChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; });
		ToggleButtonAction.ExecuteAction.BindLambda([&] {IsChecked = !IsChecked; });

		Toolbar.AddToolBarButton(ToggleButtonAction, NAME_None, LOCTEXT("ToolBarButtonToggle", "Toggle"), FText::GetEmpty(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.box-perspective"), EUserInterfaceActionType::ToggleButton);
	
		FUIAction ComboAction;
		Toolbar.AddComboButton(ComboAction, FOnGetContent::CreateLambda(
			[&]()
			{
				FMenuBuilder Menu(true, nullptr);
				MakeMenuEntries(Menu);

				return Menu.MakeWidget();
			}),
			LOCTEXT("Dropdown","Dropdown"),
			FText::GetEmpty(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.box-perspective")
		);

		if(bIncludeSettingsButtons)
		{
			Toolbar.BeginSection("SettingsTest");
			{
				Toolbar.AddToolBarButton(ButtonAction, NAME_None, LOCTEXT("ToolBarButton1", "Button"), FText::GetEmpty(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.box-perspective"));

				Toolbar.AddComboButton(
					FUIAction(),
					FOnGetContent::CreateLambda(
						[&]()
						{
							FMenuBuilder Menu(true, nullptr);
							MakeMenuEntries(Menu);

							return Menu.MakeWidget();
						}),
					TAttribute<FText>(),
							TAttribute<FText>(),
							TAttribute<FSlateIcon>(),
							true
							);


				Toolbar.AddToolBarButton(ToggleButtonAction, NAME_None, LOCTEXT("ToolBarButtonToggle", "Toggle"), FText::GetEmpty(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.box-perspective"), EUserInterfaceActionType::ToggleButton);

				Toolbar.AddComboButton(
					FUIAction(),
					FOnGetContent::CreateLambda(
						[&]()
						{
							FMenuBuilder Menu(true, nullptr);
							MakeMenuEntries(Menu);

							return Menu.MakeWidget();
						}),
					TAttribute<FText>(),
							TAttribute<FText>(),
							TAttribute<FSlateIcon>(),
							true
							);


			}
			Toolbar.EndSection();
		}

		return Toolbar.MakeWidget();
	}

    TSharedPtr<STextBlock> ComboBoxTitleBlock;

    TSharedRef<SWidget> ConstructWidgetGallery()
    {

        // initialize state variables
        RadioChoice = 3;

        SegmentedBoxChoice = 2;

        ComboItems.Add(MakeShareable(new FString("Option A One")));
        ComboItems.Add(MakeShareable(new FString("Option A Two")));
        ComboItems.Add(MakeShareable(new FString("Option A Three")));
        ComboItems.Add(MakeShareable(new FString("Option A Four")));
        ComboItems.Add(MakeShareable(new FString("Option A Five")));
        ComboItems.Add(MakeShareable(new FString("Option A Six")));
        ComboItems.Add(MakeShareable(new FString("Option A Seven")));
        ComboItems.Add(MakeShareable(new FString("Option A Eight")));
        ComboItems.Add(MakeShareable(new FString("Option A Nine")));
        ComboItems.Add(MakeShareable(new FString("Option A Ten")));
        ComboItems.Add(MakeShareable(new FString("Option A Eleven")));
        ComboItems.Add(MakeShareable(new FString("Option A Twelve")));
        ComboItems.Add(MakeShareable(new FString("Option A Thirteen")));
        ComboItems.Add(MakeShareable(new FString("Option A Fourteen")));
        

        TSharedPtr<SGridPanel> WidgetGrid = SNew(SGridPanel);

		int32 WidgetNum = 0, RowCount = 15, Cols = 3;
		auto NextSlot = [WidgetNum, RowCount, Cols](TSharedPtr<SGridPanel> Grid, const FText& InLabel) mutable -> SHorizontalBox::FScopedWidgetSlotArguments
		{
			TSharedRef<SHorizontalBox> HBox = SNew(SHorizontalBox);

			// Checkbox to show disabled state
			Grid->AddSlot((WidgetNum / RowCount) * Cols, WidgetNum % RowCount)
				.Padding(12.f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked_Lambda([HBox] { return HBox->IsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					.OnCheckStateChanged_Lambda([HBox](ECheckBoxState NewState) { HBox->SetEnabled(NewState == ECheckBoxState::Checked); })
				];

			// Add the Label
			Grid->AddSlot((WidgetNum / RowCount) * Cols + 1, WidgetNum % RowCount)
				.Padding(24.f, 16.f, 12.f, 16.f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle("NormalFont"))
					.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.White50"))
					.Text(InLabel)
				];

			//auto& Ret = Grid->AddSlot((WidgetNum / RowCount)*2 + 1, WidgetNum % RowCount)
			Grid->AddSlot((WidgetNum / RowCount) * Cols + 2, WidgetNum % RowCount)
				.Padding(12.f, 16.f, 12.f, 16.f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				[
					HBox
				];

			++WidgetNum;

			SHorizontalBox::FScopedWidgetSlotArguments NewSlot = HBox->AddSlot();
			NewSlot.VAlign(VAlign_Center)
				.HAlign(HAlign_Fill)
				.AutoWidth();
			return MoveTemp(NewSlot);
		};


        auto LeftRightLabel = [](const FName& InIconName = FName(), const FText& InLabel = FText::GetEmpty(), const FName& InTextStyle = TEXT("ButtonText")) -> TSharedRef<SWidget>
        {
        	TSharedPtr<SHorizontalBox> HBox = SNew(SHorizontalBox);
            float Space = InIconName.IsNone() ? 0.0f : 8.f;

        	if (!InIconName.IsNone())
        	{
        		HBox->AddSlot()
	        	.AutoWidth()
	            .VAlign(VAlign_Center)
	            [
	                SNew(SImage)
	                .ColorAndOpacity(FSlateColor::UseForeground())
	                .Image(FAppStyle::Get().GetBrush(InIconName))
	            ];
        	}

        	if (!InLabel.IsEmpty())
        	{
        		HBox->AddSlot()	
        		 .VAlign(VAlign_Center)
	            .Padding(Space, 0.5f, 0.f, 0.f)  // Compensate down for the baseline since we're using all caps
	            .AutoWidth()
	            [
	                SNew(STextBlock)
	                .TextStyle( &FAppStyle::Get().GetWidgetStyle< FTextBlockStyle >( InTextStyle ))
	                .Justification(ETextJustify::Center)
	                .Text(InLabel)
	            ];
        	}

        	return SNew(SBox).HeightOverride(16.f)[ HBox.ToSharedRef() ];
        };

        // SButton Primary Rounded
        NextSlot(WidgetGrid, LOCTEXT("SButtonPrimaryExampleLabelRounded", "Primary Button"))
        [
            SNew(SHorizontalBox)
            +SHorizontalBox::Slot()
            .AutoWidth()
            .Padding(8.f, 0.0f)
            .VAlign(VAlign_Center)
            [

                SNew(SButton)
                .ButtonStyle( &FAppStyle::Get().GetWidgetStyle< FButtonStyle >( "PrimaryButton" ) )
                .TextStyle( &FAppStyle::Get().GetWidgetStyle< FTextBlockStyle >("ButtonText"))
                [
                    LeftRightLabel(NAME_None, LOCTEXT("Label", "Label"))
                ]
            ]

            +SHorizontalBox::Slot()
            .AutoWidth()
            .Padding(8.f, 0.0f)
            .VAlign(VAlign_Center)
            [
                SNew(SButton)
                .ButtonStyle( &FAppStyle::Get().GetWidgetStyle< FButtonStyle >( "PrimaryButton" ) )
                [
                    LeftRightLabel("Icons.box-perspective", LOCTEXT("Label", "Label"))
                ]
            ]

            +SHorizontalBox::Slot()
            .AutoWidth()
            .VAlign(VAlign_Center)
            .Padding(8.f, 0.0f)
            [
                SNew(SButton)
                .ButtonStyle( &FAppStyle::Get().GetWidgetStyle< FButtonStyle >( "PrimaryButton" ) )
                .VAlign(VAlign_Center)
                [
                    LeftRightLabel("Icons.box-perspective")
                ]
            ]
        ];

        // SButton Rounded
        NextSlot(WidgetGrid, LOCTEXT("SButtonLabelRounded", "Default/Secondary Button"))
        [

            SNew(SHorizontalBox)
            +SHorizontalBox::Slot()
            .AutoWidth()
            .Padding(8.f, 0.0f)
            .VAlign(VAlign_Center)
            [
                SNew(SButton)
                .ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>( "Button" ))
                [
                    LeftRightLabel(NAME_None, LOCTEXT("Label", "Label"), "SmallButtonText")
                ]
            ]

            +SHorizontalBox::Slot()
            .AutoWidth()
            .Padding(8.f, 0.0f)
            [
                SNew(SButton)
                .ButtonStyle( &FAppStyle::Get().GetWidgetStyle< FButtonStyle >( "Button" ) )
                [
                    LeftRightLabel("Icons.box-perspective", LOCTEXT("Label", "Label"), "SmallButtonText")
                ]
            ]

            +SHorizontalBox::Slot()
            .AutoWidth()
            .Padding(8.f, 0.0f)
            [
                SNew(SButton)
                .ButtonStyle( &FAppStyle::Get().GetWidgetStyle< FButtonStyle >( "Button" ) )
                .VAlign(VAlign_Center)
                [
                    LeftRightLabel("Icons.box-perspective")
                ]
            ]
        ];


        // NoBorder Button Button 
        NextSlot(WidgetGrid, LOCTEXT("TextButton", "Simple Button"))
        [
            SNew(SHorizontalBox)
            +SHorizontalBox::Slot()
            .AutoWidth()
            .Padding(8.f, 0.0f)
            .VAlign(VAlign_Center)
            [
                SNew(SButton)
                .ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
                [
                    LeftRightLabel(NAME_None, LOCTEXT("Label", "Label"), "SmallButtonText")
                ]
            ]

            +SHorizontalBox::Slot()
            .AutoWidth()
            .Padding(8.f, 0.0f)
            [
                SNew(SButton)
                .ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
                [
                    LeftRightLabel("Icons.box-perspective", LOCTEXT("Label", "Label"), "SmallButtonText")
                ]
            ]

            +SHorizontalBox::Slot()
            .AutoWidth()
            .Padding(8.f, 0.0f)
            [
                SNew(SButton)
                .ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
                [
                    LeftRightLabel("Icons.box-perspective")
                ]
            ]

        ];

        // Toggle Button with Words
        NextSlot(WidgetGrid, LOCTEXT("ToggleButton", "Toggle Button"))
        [
            SNew(SHorizontalBox)

            +SHorizontalBox::Slot()
            .AutoWidth()
            .Padding(8.f)
            [
                SNew(SCheckBox)
                .Style( &FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckBox"))
                .IsChecked(ECheckBoxState::Checked)
                [
                    LeftRightLabel(NAME_None, LOCTEXT("Label", "Label"), "SmallButtonText")
                ]
            ]

            +SHorizontalBox::Slot()
            .AutoWidth()
            .Padding(8.f)
            [
                SNew(SCheckBox)
                .Style( &FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckBox"))
                .IsChecked(ECheckBoxState::Checked)
                [
                    LeftRightLabel("Icons.pyramid", LOCTEXT("Label", "Label"), "SmallButtonText")
                ]
            ]

            +SHorizontalBox::Slot()
            .AutoWidth()
            .Padding(8.f)
            [
                SNew(SCheckBox)
                .Style( &FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckBox"))
                [
                    LeftRightLabel("Icons.cylinder")
                ]
            ]
        ];

   
        // Segmented Control - Simple version lets you specify text and brush
        NextSlot(WidgetGrid, LOCTEXT("SegmentedControl", "SegmentedControl")) 
        [
        	SNew(SSegmentedControl<int32>)
            .Value(0) // InitialValue
            // .OnValueChanged_Lambda( [this] (int32 InValue) { SegmentedBoxChoice = InValue; } ) 
            // .Value_Lambda( [this] { return SegmentedBoxChoice; } )  // Bound 

        	+SSegmentedControl<int32>::Slot(0)
            .Icon(FAppStyle::Get().GetBrush("Icons.box-perspective"))
            .Text(LOCTEXT("Box", "Box"))

            +SSegmentedControl<int32>::Slot(1)
            .Icon(FAppStyle::Get().GetBrush("Icons.cylinder"))
            .Text(LOCTEXT("Cylinder", "Cylinder"))

            +SSegmentedControl<int32>::Slot(2)
            .Icon(FAppStyle::Get().GetBrush("Icons.pyramid"))
            .Text(LOCTEXT("Pyramid", "Pyramid"))

        	+SSegmentedControl<int32>::Slot(3)
            .Icon(FAppStyle::Get().GetBrush("Icons.sphere"))
            .Text(LOCTEXT("Sphere", "Sphere"))

        ];
   
        // Segmented Control - Explicitly Specify Contents
        NextSlot(WidgetGrid, LOCTEXT("SegmentedControlWithChildren", "SegmentedControl Alt")) 
        [
            SNew(SSegmentedControl<int32>)
            .Value(2)

            +SSegmentedControl<int32>::Slot(0) [ LeftRightLabel("Icons.box-perspective") ]
            +SSegmentedControl<int32>::Slot(1) [ LeftRightLabel("Icons.cylinder") ]
            +SSegmentedControl<int32>::Slot(2) [ LeftRightLabel("Icons.pyramid") ]
            +SSegmentedControl<int32>::Slot(3) [ LeftRightLabel("Icons.sphere") ]
        ];

        // SCheckBox
        NextSlot(WidgetGrid, LOCTEXT("SCheckBoxLabel", "Check Box"))
        [
            SNew(SVerticalBox)

            +SVerticalBox::Slot()
            [
                SNew(SHorizontalBox)
                + SHorizontalBox::Slot().Padding(8.f) [ SNew(SCheckBox).IsChecked(ECheckBoxState::Unchecked) ]
                + SHorizontalBox::Slot().Padding(8.f) [ SNew(SCheckBox).IsChecked(ECheckBoxState::Unchecked) ]
                + SHorizontalBox::Slot().Padding(8.f) [ SNew(SCheckBox).IsChecked(ECheckBoxState::Checked) ]
                + SHorizontalBox::Slot().Padding(8.f) [ SNew(SCheckBox).IsChecked(ECheckBoxState::Undetermined) ]
            ]

        ];

        // SCheckBox (as radio button)
        TSharedPtr<SHorizontalBox> RadioBox = SNew(SHorizontalBox);
        for (int i = 0; i < 5; i++)
        {
            RadioBox->AddSlot()
            .Padding(8.f)
            [
                SNew(SCheckBox).Style(FAppStyle::Get(), "RadioButton")
                .IsChecked_Lambda( [this, i] () -> ECheckBoxState { return RadioChoice == i ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
                .OnCheckStateChanged_Lambda( [this, i] (ECheckBoxState InState) { if (InState == ECheckBoxState::Checked ) RadioChoice = i; } )
            ];
        }

        RadioBox->AddSlot()
        .Padding(8.f)
        [
            SNew(STextBlock).Text_Lambda( [this] () { return FText::AsNumber(RadioChoice); } )
        ];

        NextSlot(WidgetGrid, LOCTEXT("SRadioButtonLabel", "Radio Button"))
        [
            SNew(SVerticalBox)

            +SVerticalBox::Slot()
            [
                RadioBox.ToSharedRef()
            ]
        ];


         // SComboBox Simple Icon Only version 
        NextSlot(WidgetGrid, LOCTEXT("SComboBoxIconLabel", "SimpleComboBox"))
        [
            SNew(SComboBox<TSharedPtr<FString> >)
            .ComboBoxStyle( FAppStyle::Get(), "SimpleComboBox")
            .OptionsSource(&ComboItems)
            .OnGenerateWidget_Lambda([](TSharedPtr<FString> Item) 
            { 
                return SNew(STextBlock).Text( FText::FromString(*Item));
            } )
            [
                SNew(SImage)
                .ColorAndOpacity(FSlateColor::UseForeground())
                .Image(FAppStyle::Get().GetBrush("Icons.box-perspective"))
            ]

        ];


        // SComboBox
        NextSlot(WidgetGrid, LOCTEXT("SComboBoxLabel", "ComboBox"))
        .AutoWidth()
        // .FillWidth(1.0)
        [
            SNew(SComboBox<TSharedPtr<FString> >)
            .OptionsSource(&ComboItems)
            .OnGenerateWidget_Lambda([](TSharedPtr<FString> Item) 
            { 
                return SNew(STextBlock).Text( FText::FromString(*Item));
            } )
            .OnSelectionChanged_Lambda([this] (TSharedPtr<FString> InSelection, ESelectInfo::Type InSelectInfo) 
            {
                if (InSelection.IsValid() && ComboBoxTitleBlock.IsValid())
                {
                    ComboBoxTitleBlock->SetText( FText::FromString(*InSelection));
                }
            } )
            [
                SAssignNew(ComboBoxTitleBlock, STextBlock).Text(LOCTEXT("ComboLabel", "Label"))  
            ]

        ];

       
     
        /*
        // SHeader
        NextSlot(WidgetGrid, LOCTEXT("SHeaderLabel", "SHeader"))
        [
            SNew(SHeader)
                .Content()
                [
                    SNew(STextBlock)
                        .Text(LOCTEXT("HeaderContentLabel", "Header Content"))
                ]
        ];

        // SHyperlink
        NextSlot(WidgetGrid, LOCTEXT("SHyperlinkLabel", "SHyperlink"))
        [
            SNew(SHyperlink)
                .Text(LOCTEXT("SHyperlinkText", "Text to appear in the Hyperlink widget."))

        ];

        /// SBreadcrumbTrailLabel
        NextSlot(WidgetGrid, LOCTEXT("SBreadcrumbTrailLabel", "SBreadcrumbTrail"))
        [
            SNew(SHorizontalBox)
            + SHorizontalBox::Slot()
                .FillWidth(1.0f)
                [
                    SAssignNew(BreadcrumbTrail, SBreadcrumbTrail<int32>)

                ]

            + SHorizontalBox::Slot()
                .AutoWidth()
                [
                    SNew(SButton)
                        .Text(LOCTEXT("AddBreadCrumbLabel", "Add"))
                        .HAlign(HAlign_Center)
                        .VAlign(VAlign_Center)
                        .OnClicked(this, &SWidgetGallery::HandleBreadcrumbTrailAddButtonClicked)
                ]
        ];


        // SCircularThrobber
        NextSlot(WidgetGrid, LOCTEXT("SCircularThrobberLabel", "SCircularThrobber"))
        [
            SNew(SCircularThrobber)
        ];

        // SColorBlock
        NextSlot(WidgetGrid, LOCTEXT("SColorBlockLabel", "SColorBlock"))
        [
            SNew(SColorBlock)
                .Color(FLinearColor(1.0f, 0.0f, 0.0f))
        ];
        */

        /*
        // SImage
        WidgetGrid->AddSlot(0, 16)
            [
                SNew(STextBlock)
                    .Text(LOCTEXT("SImageLabel", "SImage"))
            ]

        WidgetGrid->AddSlot(1, 16)
            .Padding(0.0f, 5.0f)
            [
                SNew(SImage)
                    .Image(FTestStyle::Get().GetBrush(TEXT("NewLevelBlank")))           
            ]
            */
/*
        // SProgressBar
        NextSlot(WidgetGrid, LOCTEXT("SProgressBarLabel", "SProgressBar"))
        [
            SNew(SVerticalBox)

            + SVerticalBox::Slot()
                .AutoHeight()
                [
                    SNew(SProgressBar)
                        .Percent(this, &SWidgetGallery::HandleProgressBarPercent)
                ]

            + SVerticalBox::Slot()
                .AutoHeight()
                [
                    SNew(SProgressBar)
                ]
        ];
    */
/*
        // SRotatorInputBox
        WidgetGrid->AddSlot(0, 19)
            [
                SNew(STextBlock)
                    .Text(LOCTEXT("SRotatorInputBoxLabel", "SRotatorInputBox"))
            ]

        WidgetGrid->AddSlot(1, 19)
            .Padding(0.0f, 5.0f)
            [
                SNew(SRotatorInputBox)
                    .Roll(0.5f)
                    .Pitch(0.0f)
                    .Yaw(1.0f)          
            ]

       // SSeparator
        NextSlot(WidgetGrid, LOCTEXT("SSeparatorLabel", "SSeparator"))
        [
            SNew(SBox)
                .HeightOverride(100.0f)
                .WidthOverride(150.0f)
                [
                    SNew(SHorizontalBox)

                    + SHorizontalBox::Slot()
                        .VAlign(VAlign_Center)
                        .FillWidth(0.75f)
                        [
                            SNew(SSeparator)
                                .Orientation(Orient_Horizontal)
                        ]

                    + SHorizontalBox::Slot()
                        .HAlign(HAlign_Center)
                        .FillWidth(0.25f)
                        [
                            SNew(SSeparator)
                                .Orientation(Orient_Vertical)
                        ]
                ]           
        ];
        */

        // SSlider
        NextSlot(WidgetGrid, LOCTEXT("SSliderLabel", "SSlider"))
        .FillWidth(1.0)
        [
            SNew(SSlider)
            // .IndentHandle(false)
            .Orientation(Orient_Horizontal)
            .Value(0.5f)

                /*
            + SHorizontalBox::Slot()
                .HAlign(HAlign_Center)
                .FillWidth(0.25f)
                [
                    SNew(SSlider)
                        .Orientation(Orient_Vertical)
                        .Value(0.5f)
                ]
            */
        ];

            /*
        // SSlider (no indentation)
        NextSlot(WidgetGrid, LOCTEXT("SSliderNoIndentLabel", "SSlider (no indentation)"))
        .FillWidth(1.0)
        [
            SNew(SSlider)
            .IndentHandle(false)
            .Orientation(Orient_Horizontal)
            .Value(0.5f)
        ];

        // SSpacer
        NextSlot(WidgetGrid, LOCTEXT("SSpacerLabel", "SSpacer"))
        [
            SNew(SSpacer)
                .Size(FVector2D(100, 100))      
        ];

        // SSpinningImage
        WidgetGrid->AddSlot(0, 25)
            [
                SNew(STextBlock)
                    .Text(LOCTEXT("SSpinningImageLabel", "SSpinningImage"))
            ]

        WidgetGrid->AddSlot(1, 25)
            .HAlign(HAlign_Left)
            .Padding(0.0f, 5.0f)
            [
                SNew(SSpinningImage)
                    .Image(FTestStyle::Get().GetBrush("TestRotation16px"))
            ]
        */

        // SSpinBox
        NextSlot(WidgetGrid, LOCTEXT("SSpinBoxLabel", "SSpinBox"))
        .FillWidth(1.0)
        [
            SNew(SBox)
            .MinDesiredWidth(220)
            [
                SNew(SSpinBox<float>)
                .MinValue(0.0f)
                .MaxValue(500.0f)
                .MinSliderValue(TAttribute< TOptional<float> >(-500.0f))
                .MaxSliderValue(TAttribute< TOptional<float> >(500.0f))
                .Value(123.0456789)
                .Delta(0.5f)
            ]
        ];

        // SNumericEntrySpinBox
        NextSlot(WidgetGrid, LOCTEXT("SNumericEntryBoxLabel", "SNumericEntryBox"))
        .FillWidth(1.0)
        [
            SNew(SNumericEntryBox<float>)
            .MinValue(-1000.0f)
            .MaxValue(1000.0f)
            .MinSliderValue(TAttribute< TOptional<float> >(-500.0f))
            .MaxSliderValue(TAttribute< TOptional<float> >(500.0f))
            .Delta(0.5f)
            .Value(500.0f)
            .AllowSpin(true)

            .OnValueChanged_Lambda( [this] (float InValue) { NumericEntryBoxChoice = InValue; } )
            .OnValueCommitted_Lambda( [this] (float InValue, ETextCommit::Type CommitInfo) { NumericEntryBoxChoice = InValue; } )
            .Value_Lambda( [this] { return TOptional<float>(NumericEntryBoxChoice); } )
        ];

        // SNumericEntrySpinBox
        NextSlot(WidgetGrid, LOCTEXT("SNumericEntryBoxNoSpinLabel", "SNumericEntryBox (No Spin)"))
        .FillWidth(1.0)
        [

            SNew(SNumericEntryBox<float>)
            .MinValue(-1000.0f)
            .MaxValue(1000.0f)
            .MinSliderValue(TAttribute< TOptional<float> >(-500.0f))
            .MaxSliderValue(TAttribute< TOptional<float> >(500.0f))
            .Delta(0.5f)
            .Value(500.0f)

            .OnValueChanged_Lambda( [this] (float InValue) { NumericEntryBoxChoice = InValue; } )
            .OnValueCommitted_Lambda( [this] (float InValue, ETextCommit::Type CommitInfo) { NumericEntryBoxChoice = InValue; } )
            .Value_Lambda( [this] { return TOptional<float>(NumericEntryBoxChoice); } )
        ];

        // SEditableText
        NextSlot(WidgetGrid, LOCTEXT("SEditableTextLabel", "SEditableText"))
        .FillWidth(1.0)
        [
            SNew(SEditableText)
            .HintText(LOCTEXT("SEditableTextHint", "This is editable text"))
        ];

        // SEditableTextBox
        NextSlot(WidgetGrid, LOCTEXT("SEditableTextBoxLabel", "SEditableTextBox"))
        .FillWidth(1.0)
        [
            SNew(SEditableTextBox)
            .HintText(LOCTEXT("SEditableTextBoxHint", "This is an editable text box"))
        ];


        // SMultiLineEditableText
        NextSlot(WidgetGrid, LOCTEXT("SMultiLineEditableTextLabel", "SMultiLineEditableText"))
        .FillWidth(1.0)
        [
            SNew(SBox).MinDesiredHeight(48.f).MinDesiredWidth(200.f)
            [
                SNew(SMultiLineEditableText)
                .HintText(LOCTEXT("SMultiLineEditableTextHint", "This is multi-line \n\t\t\t editable text"))
		.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
            ]
        ];

        // SMultiLineEditableTextBox
        NextSlot(WidgetGrid, LOCTEXT("SMultiLineEditableTextBoxLabel", "SMultiLineEditableTextBox"))
        .FillWidth(1.0)
        [
            SNew(SBox).MinDesiredHeight(48.f).MinDesiredWidth(200.f)
            [
                SNew(SMultiLineEditableTextBox)
                .HintText(LOCTEXT("SMultiLineEditableTextBoxHint", "This is a multi-line editable text box"))
		.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
            ]    
        ];


        // SInlineEditableTextBlock
        NextSlot(WidgetGrid, LOCTEXT("SInlineLineEditableTextLabel", "SInlineEditableText"))
        .FillWidth(1.0)
        [
            SNew(SInlineEditableTextBlock)
            .Text(LOCTEXT("SInlineEditableTextHint", "Inline Editable Text"))
        ];


        // SSearchBox
        NextSlot(WidgetGrid, LOCTEXT("SSearchBoxLabel", "SSearchBox"))
        .FillWidth(1.0)
        [
            SNew(SSearchBox)
        ];

		// SProgressBar
		NextSlot(WidgetGrid, LOCTEXT("SSProgressBar", "SProgressBar"))
			.FillWidth(1.0)
			[
				SNew(SProgressBar)
				.Percent(.5f)
			];

		// SProgressBar marquee
		NextSlot(WidgetGrid, LOCTEXT("SSProgressBarMarquee", "SProgressBar marquee"))
			.FillWidth(1.0)
			[
				SNew(SProgressBar)
			];

        // Window Flash Button
        NextSlot(WidgetGrid, LOCTEXT("WindowFlashAction", "Window Flash Button"))
        [
            SNew(SButton)
            .Text(LOCTEXT("WindowFlashLabel", "Window Flash"))
            .OnClicked_Lambda( [] () -> FReply { FSlateApplication::Get().GetActiveTopLevelWindow()->FlashWindow(); return FReply::Handled(); } )

        ];


		auto NewMenuDelegate = FNewMenuDelegate::CreateLambda(
			[&](FMenuBuilder& MenuBuilder)
			{
				MenuBuilder.BeginSection(NAME_None, LOCTEXT("MenuHeader1", "Menu Header 1"));
				MakeMenuEntries(MenuBuilder);
				MenuBuilder.AddSubMenu(LOCTEXT("SubMenu1", "Sub Menu 1"), FText::GetEmpty(), FNewMenuDelegate::CreateRaw(this, &SStarshipGallery::MakeMenuEntries));
				MenuBuilder.EndSection();

				MenuBuilder.BeginSection(NAME_None, LOCTEXT("MenuHeader2", "Menu Header 2"));
				MenuBuilder.AddMenuEntry(LOCTEXT("MenuEntry5", "Menu Entry 5"), FText::GetEmpty(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.box-perspective"), FUIAction());
				MenuBuilder.AddMenuEntry(LOCTEXT("MenuEntry6", "Menu Entry 6"), FText::GetEmpty(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.box-perspective"), FUIAction());
				MenuBuilder.AddMenuEntry(LOCTEXT("MenuEntry7", "Menu Entry 7"), FText::GetEmpty(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.box-perspective"), FUIAction());
				MenuBuilder.AddMenuEntry(LOCTEXT("MenuEntry8", "Menu Entry 8"), FText::GetEmpty(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.box-perspective"), FUIAction());
				MenuBuilder.EndSection();
			});

		FMenuBarBuilder MenuBar(nullptr);

		MenuBar.AddPullDownMenu(LOCTEXT("Menu1", "Menu 1"), FText::GetEmpty(), NewMenuDelegate);
		MenuBar.AddPullDownMenu(LOCTEXT("Menu2", "Menu 2"), FText::GetEmpty(), NewMenuDelegate);

		NextSlot(WidgetGrid, LOCTEXT("MenuBar", "Menu Bar"))
		[
			MenuBar.MakeWidget()
		];


		NextSlot(WidgetGrid, LOCTEXT("Toolbar", "Toolbar"))
		[
			BuildToolbar(true)
		];

		NextSlot(WidgetGrid, LOCTEXT("Toolbar2", "Toolbar 2"))
		[
			BuildToolbar(false)
		];

        return SNew(SBorder)
        .BorderImage( FAppStyle::Get().GetBrush("ToolPanel.GroupBorder") )
        [
            SNew(SScrollBox)

            + SScrollBox::Slot()
            .Padding(48.0f)
            [
                WidgetGrid.ToSharedRef()
            ]
        ];

    }

	class SListGalleryWidget : public SCompoundWidget
	{
	private:
		TArray<TSharedPtr<int32>> ListViewItemsInArray;
		UE::Slate::Containers::TObservableArray<TSharedPtr<int32>> ListViewItemsInObservableArray;
		TSharedPtr<UE::Slate::Containers::TObservableArray<TSharedPtr<int32>>> SharedListViewItemsInObservableArray;
		TSharedPtr<SListView<TSharedPtr<int32>>> ListView_List;
		TSharedPtr<STileView<TSharedPtr<int32>>> ListView_Tile;
		TSharedPtr<STreeView<TSharedPtr<int32>>> ListView_Tree;

	public:
		SLATE_BEGIN_ARGS(SListGalleryWidget){}
		SLATE_END_ARGS()
		void Construct(const FArguments&)
		{
			TSharedRef<SHorizontalBox> TheBox = SNew(SHorizontalBox);

			SharedListViewItemsInObservableArray = MakeShared<UE::Slate::Containers::TObservableArray<TSharedPtr<int32>>>();
			for (int32 Index = 0; Index < 15; ++Index)
			{
				ListViewItemsInArray.Add(MakeShared<int32>(Index));
				ListViewItemsInObservableArray.Add(MakeShared<int32>(Index));
				SharedListViewItemsInObservableArray->Add(MakeShared<int32>(Index));
			}

			TheBox->AddSlot()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				[
					SNew(SButton)
					.OnClicked_Lambda([this]()
					{
						ListViewItemsInArray.Add(MakeShared<int32>(ListViewItemsInArray.Num()));
						ListViewItemsInObservableArray.Add(MakeShared<int32>(ListViewItemsInObservableArray.Num()));
						SharedListViewItemsInObservableArray->Add(MakeShared<int32>(SharedListViewItemsInObservableArray->Num()));
						return FReply::Handled();
					})
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("Add", "Add"))
					]
				]
				+ SVerticalBox::Slot()
				[
					SNew(SButton)
					.OnClicked_Lambda([this]()
					{
						int32 Rand = FMath::RandHelper(ListViewItemsInArray.Num());
						ListViewItemsInArray.RemoveAt(Rand);
						ListViewItemsInObservableArray.RemoveAt(Rand);
						SharedListViewItemsInObservableArray->RemoveAt(Rand);
						return FReply::Handled();
					})
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("Remove", "Remove"))
					]
				]
			];

			TheBox->AddSlot()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				[
					SAssignNew(ListView_List, SListView<TSharedPtr<int32>>)
					.ListItemsSource(&ListViewItemsInArray)
					.ListItemsSource(&ListViewItemsInObservableArray)
					.ListItemsSource(SharedListViewItemsInObservableArray)
					.OnGenerateRow_Lambda([](TSharedPtr<int32> Value, const TSharedRef<STableViewBase>& OwnerTable)
					{
						typedef STableRow<TSharedPtr<int32>> RowType;

						TSharedRef<RowType> NewRow = SNew(RowType, OwnerTable);
						NewRow->SetContent(SNew(STextBlock).Text(FText::AsNumber(*Value.Get())));

						return NewRow;
					})
				]
				+ SVerticalBox::Slot()
				[
					SAssignNew(ListView_Tile, STileView<TSharedPtr<int32>>)
					.ListItemsSource(&ListViewItemsInArray)
					.ListItemsSource(&ListViewItemsInObservableArray)
					.ListItemsSource(SharedListViewItemsInObservableArray)
					.OnGenerateTile_Lambda([](TSharedPtr<int32> Value, const TSharedRef<STableViewBase>& OwnerTable)
					{
						typedef STableRow<TSharedPtr<int32>> RowType;

						TSharedRef<RowType> NewRow = SNew(RowType, OwnerTable);
						NewRow->SetContent(SNew(STextBlock).Text(FText::AsNumber(*Value.Get())));

						return NewRow;
					})
				]
				+ SVerticalBox::Slot()
				[
					SAssignNew(ListView_Tree, STreeView<TSharedPtr<int32>>)
					.TreeItemsSource(&ListViewItemsInArray)
					.TreeItemsSource(&ListViewItemsInObservableArray)
					.TreeItemsSource(SharedListViewItemsInObservableArray)
					.OnGenerateRow_Lambda([](TSharedPtr<int32> Value, const TSharedRef<STableViewBase>& OwnerTable)
					{
						typedef STableRow<TSharedPtr<int32>> RowType;

						TSharedRef<RowType> NewRow = SNew(RowType, OwnerTable);
						NewRow->SetContent(SNew(STextBlock).Text(FText::AsNumber(*Value.Get())));

						return NewRow;
					})
					.OnGetChildren_Lambda([](TSharedPtr<int32> Parent, TArray<TSharedPtr<int32>>& OutParent)
					{
						OutParent.Add(MakeShared<int32>(99));
					})
				]
			];

			ChildSlot[TheBox];
		}
		~SListGalleryWidget()
		{
			if (ListView_List)
			{
				ListView_List->ClearItemsSource();
				ListView_List.Reset();
			}
			if (ListView_Tile)
			{
				ListView_Tile->ClearItemsSource();
				ListView_Tile.Reset();
			}
			if (ListView_Tree)
			{
				ListView_Tree->ClearRootItemsSource();
				ListView_Tree.Reset();
			}
			ChildSlot[SNullWidget::NullWidget];
		}
	};
	
	TSharedRef<SWidget> ConstructListGallery()
	{
		return SNew(SListGalleryWidget);
	}

	TArray<TSharedPtr<EHorizontalAlignment>> HorizontalAlignmentComboItems;

}; // class SStarshipGallery

    
/**
 * Creates a new widget gallery.
 *
 * @return The new gallery widget.
 */
TSharedRef<SWidget> MakeStarshipGallery()
{
    return SNew(SStarshipGallery);
};

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef LOCTEXT_NAMESPACE

#endif // #if !UE_BUILD_SHIPPING
