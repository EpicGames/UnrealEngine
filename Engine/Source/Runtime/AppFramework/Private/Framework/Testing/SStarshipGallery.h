// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"

#if !UE_BUILD_SHIPPING

/** @return a new Starship Widget Test Gallery */
TSharedRef<SWidget> MakeStarshipGallery();

#endif // #if !UE_BUILD_SHIPPING

