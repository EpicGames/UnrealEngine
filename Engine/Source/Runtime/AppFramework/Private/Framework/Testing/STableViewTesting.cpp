// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Testing/STableViewTesting.h"
#include "Layout/Margin.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/CoreStyle.h"
#include "Layout/WidgetPath.h"
#include "SlateOptMacros.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STileView.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SCheckBox.h"

#if !UE_BUILD_SHIPPING

#define LOCTEXT_NAMESPACE "STableViewTesting"

class FTestData;

/** Simple DragDropOperation for reordering items in this example's lists and trees. */
class FTableViewDragDrop : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FTableViewDragDrop, FDragDropOperation);

	static TSharedRef<FTableViewDragDrop> New(const TSharedRef<FTestData>& InTestData)
	{
		return MakeShareable(new FTableViewDragDrop(InTestData));
	}

	const TSharedRef<FTestData>& GetDraggedItem() const { return TestData; }

private:
	FTableViewDragDrop(const TSharedRef<FTestData>& InTestData)
	: TestData( InTestData )
	{
	}

	TSharedRef<FTestData> TestData;
};

/**
 * A data item with which we can test lists and trees.
 * It supports having children and having a name.
 */
class FTestData
{
	public: 
		static TSharedRef<FTestData> Make( const FText& ChildName )
		{
			return MakeShareable( new FTestData( ChildName, MakeRandomItemHeight() ) );
		}

		/**
		 * Recursively look for `ItemToRemove` in `RemoveFrom` and any of the descendants, and remove `ItemToRemove`.
		 * @return true when successful.
		 */
		static bool RemoveRecursive(TArray< TSharedPtr< FTestData > >& RemoveFrom, const TSharedPtr<FTestData>& ItemToRemove)
		{
			int32 ItemIndex = RemoveFrom.Find(ItemToRemove);
			if (ItemIndex != INDEX_NONE)
			{
				RemoveFrom.RemoveAt(ItemIndex);
				return true;
			}

			// Did not successfully remove an item. Try all the children.
			for (ItemIndex = 0; ItemIndex < RemoveFrom.Num(); ++ItemIndex)
			{
				if (RemoveRecursive(RemoveFrom[ItemIndex]->Children, ItemToRemove))
				{
					return true;
				}
			}

			return false;
		}

		/**
		 * Recursively look for `TargetItem` in `InsertInto` and any of the descendants.
		 * Insert `ItemToInsert` relative to the target.
		 * Relative positioning dictated by `RelativeLocation`.
		 *
		 * @return true when successful.
		 */
		static bool InsertRecursive(TArray< TSharedPtr< FTestData > >& InsertInto, const TSharedRef<FTestData>& ItemToInsert, const TSharedRef<FTestData>& TargetItem, EItemDropZone RelativeLocation)
		{
			const int32 TargetIndex = InsertInto.Find(TargetItem);
			if (TargetIndex != INDEX_NONE)
			{
				if (RelativeLocation == EItemDropZone::AboveItem)
				{
					InsertInto.Insert(ItemToInsert, TargetIndex);
				}
				else if (RelativeLocation == EItemDropZone::BelowItem)
				{
					InsertInto.Insert(ItemToInsert, TargetIndex + 1);
				}
				else
				{
					ensure(RelativeLocation == EItemDropZone::OntoItem);
					InsertInto[TargetIndex]->Children.Insert(ItemToInsert, 0);
				}				
				return true;
			}

			// Did not successfully remove an item. Try all the children.
			for (int32 ItemIndex = 0; ItemIndex < InsertInto.Num(); ++ItemIndex)
			{
				if (InsertRecursive(InsertInto[ItemIndex]->Children, ItemToInsert, TargetItem, RelativeLocation))
				{
					return true;
				}
			}

			return false;
		}

		static float MakeRandomItemHeight()
		{
			return static_cast<float>(rand()) / static_cast<float>(RAND_MAX) * 50.0f;
		}

		/**
		 * @return true or false with decreasing probability of true for higher values of Level.
		 * Level < 0 is always true.
		 * Level = 0 is 50.0% chance of true.
		 * Level = 1 is 25.0% chance of true.
		 * Level = 2 is 12.5% chance of true.
		 * Level = N is 1/(2^Level) chance of true.
		 */
		static bool BinaryProbability( int32 Level )
		{
			if (Level < 0)
			{
				return true;
			}
			else if ( rand() < RAND_MAX / 2 )
			{
				return BinaryProbability(Level - 1);
			}
			else
			{
				return false;
			}
		}

		/**
		 * Recursively add a bunch of descendants to this node. TotalDescendants is how many descendants we should add before we stop.
		 *
		 * @param MakeChildrenForMe       The node to which to add children
		 * @param DescendantsLeftToMake   How many descendants we still need to make
		 * @param NestingDepth            Track the nesting depth to prevent super-deep nesting
		 *
		 * @return How many descendants we still need to make after this function ran.
		 */
		static int32 GenerateChildren( TSharedRef<FTestData> MakeChildrenForMe, int32 DescendantsLeftToMake, int32 NestingDepth )
		{
			for ( ; DescendantsLeftToMake >= 0; )
			{
				TSharedRef<FTestData> NewChild = FTestData::Make( LOCTEXT("ChildItem", "Child Item" ) );
				MakeChildrenForMe->AddChild( NewChild );
				--DescendantsLeftToMake;
				
				// Should we stop adding to this level and pop back up? Max out at 5 levels of nesting.
				const bool bAscend = BinaryProbability(5-NestingDepth);
				// Should we descend to a deeper nesting level?
				const bool bDescend = !bAscend && BinaryProbability(NestingDepth);
				
				if ( bAscend )
				{
					// We're done on this level; go up
					return DescendantsLeftToMake;
				}
				else if ( bDescend )
				{
					// Descend further
					DescendantsLeftToMake = GenerateChildren( NewChild, DescendantsLeftToMake, NestingDepth+1 );	
				}
				else
				{
					// Continue adding on this level
				}
				
			}
			return DescendantsLeftToMake;
		}

	public:
		
		/**
		 * Add a child to this test data item.
		 *
		 * @param InChild  A reference to the child being added
		 */
		void AddChild( TSharedRef<FTestData> InChild )
		{
			Children.Add( InChild );
		}
		
		/** @return the name of this data item */
		const FText& GetName() const
		{
			return Name;
		}
		
		/**
		 * Set this data node's name
		 *
		 * @param InNewName  The new name to assign to this data item.
		 */
		void SetName( const FText& InNewName )
		{
			Name = InNewName;
		}
		
		/**
		 * Populate the OutChildren array with this data item's children.
		 *
		 * @param OutChildren  
		 */
		void GetChildren( TArray< TSharedPtr<FTestData> >& OutChildren )
		{
			OutChildren = Children;
		}

		/** @return the dummy number value */
		float GetNumber() const
		{
			return Number;
		}

		/** @param NewValue  The new dummy value */
		void SetNumber( float NewValue )
		{
			Number = NewValue;
		}
		
	private:
		/**
		 * Construct test data given a name.
		 *
		 * @param InName  The Name of this test data item.
		 */
		FTestData( const FText& InName, float InValue )
		: Name( InName )
		, Number( InValue )
		{		
		}

		/** This data item's children */
		TArray< TSharedPtr<FTestData> > Children;
		/** This data item's name */
		FText Name;
		/** A dummy number value */
		float Number;
};


class STestMenu : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( STestMenu ){}
	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 *
	 * @param InArgs   Declaration from which to construct the widget.
	 */
	void Construct(const FArguments& InArgs)
	{
		this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FCoreStyle::Get().GetBrush("Menu.Background"))
			.Padding( FMargin(5) )
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SButton)
					.Text(LOCTEXT("TestMenuButtonText00", "Option 00"))
					.ToolTipText(LOCTEXT("TestMenuButtonToolTip00", "The first option text."))
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(PopupAnchor, SMenuAnchor)
					.Placement(MenuPlacement_MenuRight)
					.OnGetMenuContent( this, &STestMenu::OnGetContent )
					[
						SNew(SButton)
						.Text(LOCTEXT("TestMenuButtonText01", "Option 01 >"))
						.ToolTipText(LOCTEXT("TestMenuButtonToolTip01", "The first option text."))
						.OnClicked( this, &STestMenu::OpenSubmenu )
					]			
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SButton)
					.Text(LOCTEXT("TestMenuButtonText02", "Option 02") )
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SButton)
					.Text(LOCTEXT("TestMenuButtonText03", "Option 03"))
					.ToolTipText(LOCTEXT("TestMenuButtonToolTip03", "The fourth option text."))
				]
			]
		];
	}

	TSharedPtr<SMenuAnchor> PopupAnchor;
	FReply OpenSubmenu()
	{
		PopupAnchor->SetIsOpen( ! PopupAnchor->IsOpen() );
		return FReply::Handled();
	}

	TSharedRef<SWidget> OnGetContent() const
	{
		return SNew(STestMenu);
	}
};


/**
 * An Item Editor used by list testing.
 * It visualizes a string and edits its contents.
 */
class SItemEditor : public SMultiColumnTableRow< TSharedPtr<FTestData> >
{
public:
	
	SLATE_BEGIN_ARGS( SItemEditor )
		: _ItemToEdit()
		{}

		SLATE_EVENT(FOnCanAcceptDrop, OnCanAcceptDrop)
		SLATE_EVENT(FOnAcceptDrop, OnAcceptDrop)
		SLATE_EVENT(FOnDragDetected, OnDragDetected)
		SLATE_ARGUMENT(TSharedPtr<FTestData>, ItemToEdit)

	SLATE_END_ARGS()
	

	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		FSlateFontInfo ItemEditorFont = FCoreStyle::Get().GetFontStyle(TEXT("NormalFont"));

		if ( ColumnName == TEXT("Name") )
		{
			// The name column is special. In lists it needs to show an expander arrow and be indented
			// in order to give the appearance of a list.

			TSharedRef<SWidget> CellContent =
				SNew( STextBlock )
				.Font( ItemEditorFont )
				.Text( this, &SItemEditor::OnReadText );

			if ( OwnerTablePtr.Pin()->GetTableViewMode() == ETableViewMode::Tree )
			{
				// Rows in a tree need to show an SExpanderArrow (it also indents!) to give the appearance of being a tree.
				return
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew( SExpanderArrow, SharedThis(this) )
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						CellContent
					];
			}
			else
			{
				// Lists do not need an expander arrow
				return CellContent;
			}
		}
		else if (ColumnName == TEXT("Number"))
		{
			return
				SNew(SSpinBox<float>)
				.MinValue(-5)
				.MaxValue(800)
				.Font( ItemEditorFont )
				.Value( this, &SItemEditor::OnReadValue )
				.OnValueChanged( this, &SItemEditor::OnWriteValue );
		}
		else if (ColumnName == TEXT("TextField"))
		{
			return
				SAssignNew( MyTextBox, SEditableTextBox )
					.Font( ItemEditorFont )
					.Text( this, &SItemEditor::OnReadText )
					.OnTextChanged( this, &SItemEditor::OnTextChanged );
		}
		else if (ColumnName == TEXT("TextBlock"))
		{
			return SNew( STextBlock ) .Font( ItemEditorFont ) .Text( this, &SItemEditor::OnReadText );
		}
		else if (ColumnName == TEXT("AddChild"))
		{
			return
				SNew(SBorder)
				.HAlign(HAlign_Center)
				.Padding(this, &SItemEditor::GetVariableHeight)
				[
					SNew( SButton )
					.TouchMethod( EButtonTouchMethod::PreciseTap )
					.OnClicked( this, &SItemEditor::OnAddChild )
					[
						SNew(STextBlock)
						.Text( LOCTEXT("AddChildButtonText", "+") )
					]
				];
		}
		else
		{
			return
			SNew(STextBlock)
			. Text( FText::Format( LOCTEXT("UnsupprtedColumnText", "Unsupported Column: {0}"), FText::FromName( ColumnName ) ) );
		}
	}

	/**
	 * Construct the widget
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		ItemToEdit = InArgs._ItemToEdit;
		
		FSuperRowType::Construct( FSuperRowType::FArguments()
			.OnCanAcceptDrop(InArgs._OnCanAcceptDrop)
			.OnAcceptDrop(InArgs._OnAcceptDrop)
			.OnDragDetected(InArgs._OnDragDetected)
			.Padding(0)
		, InOwnerTableView);
	}
	
	void SpawnContextMenu( const FVector2D& SpawnLocation )
	{
		FSlateApplication::Get().PushMenu
		(
			SharedThis( this ),
			FWidgetPath(),
			GetPopupContent(),
			SpawnLocation,
			FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu )
		);
	}
	
private:
	
	FMargin GetVariableHeight() const
	{
		return FMargin( 0,0,0,this->ItemToEdit->GetNumber() );
	}

	TSharedRef<SWidget> GetPopupContent()
	{		
		return SNew(STestMenu);
	}
	
	FReply OnAddChild()
	{
		ItemToEdit->AddChild( FTestData::Make( LOCTEXT("NewChild", "New Child") ) );
		return FReply::Handled();
	}	

	/** Modify the text when the widget changes it */
	void OnTextChanged( const FText& NewText )
	{
		if (ItemToEdit.IsValid())
		{
			ItemToEdit->SetName( NewText );
		}		
	}
	
	/** @return the text being edited by this widget */
	FText OnReadText() const
	{
		if (ItemToEdit.IsValid())
		{
			return ItemToEdit->GetName();
		}
		else
		{
			return FText::GetEmpty();
		}		
	}

	float OnReadValue() const
	{
		return ItemToEdit->GetNumber();
	}

	void OnWriteValue( float NewValue )
	{
		ItemToEdit->SetNumber(NewValue);
	}

	/** Text box used to edit the Data Item's name */
	TSharedPtr<SEditableTextBox> MyTextBox;
	/** A pointer to the data item that we visualize/edit */
	TSharedPtr<FTestData> ItemToEdit;
};



/**
 * An Item Editor used by tile view testing.
 * It visualizes a string and edits its contents.
 */
class STileItemEditor : public STableRow< TSharedPtr<FTestData> >
{
public:
	
	SLATE_BEGIN_ARGS( STileItemEditor )
		: _ItemToEdit()
		{}

		SLATE_EVENT(FOnCanAcceptDrop, OnCanAcceptDrop)
		SLATE_EVENT(FOnAcceptDrop, OnAcceptDrop)
		SLATE_EVENT(FOnDragDetected, OnDragDetected)
		SLATE_ARGUMENT(TSharedPtr<FTestData>, ItemToEdit)

	SLATE_END_ARGS()
	
	/**
	 * Construct the widget
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		STableRow<TSharedPtr<FTestData> >::ConstructInternal( STableRow::FArguments()
			.OnCanAcceptDrop(InArgs._OnCanAcceptDrop)
			.OnAcceptDrop(InArgs._OnAcceptDrop)
			.OnDragDetected(InArgs._OnDragDetected)
			.ShowSelection(true)
		, InOwnerTableView);

		ItemToEdit = InArgs._ItemToEdit;

		FSlateFontInfo ItemEditorFont = FCoreStyle::Get().GetFontStyle(TEXT("NormalFont"));

		this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage( FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder") )
			.BorderBackgroundColor(FLinearColor(1,1,1,0.45f))
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4)
				[
					// Name
					SNew( STextBlock )
					.Font( ItemEditorFont )
					.Text( this, &STileItemEditor::OnReadText )
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
				[
					// Number
					SNew(SSpinBox<float>)
					.MinValue(-5)
					.MaxValue(15)
					.Font( ItemEditorFont )
					.Value( this, &STileItemEditor::OnReadValue )
					.OnValueChanged( this, &STileItemEditor::OnWriteValue )
				]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				//TextField
				SNew( SEditableTextBox )
				.Font( ItemEditorFont )
				.Text( this, &STileItemEditor::OnReadText )
				.OnTextChanged( this, &STileItemEditor::OnTextChanged )
			]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
				[
					//TextBlock
					SNew( STextBlock ) .Font( ItemEditorFont ) .Text( this, &STileItemEditor::OnReadText )
				]
			]
		];
	}
	
	void SpawnContextMenu( const FVector2D& SpawnLocation )
	{
		FSlateApplication::Get().PushMenu
		(
			SharedThis( this ),
			FWidgetPath(),
			GetPopupContent(),
			SpawnLocation,
			FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu )
		);
	}
	
private:
	
	TSharedRef<SWidget> GetPopupContent()
	{		
		return SNew(STestMenu);
	}

	/** Modify the text when the widget changes it */
	void OnTextChanged( const FText& NewText )
	{
		if (ItemToEdit.IsValid())
		{
			ItemToEdit->SetName( NewText );
		}		
	}
	
	/** @return the text being edited by this widget */
	FText OnReadText() const
	{
		if ( ItemToEdit.IsValid() )
		{
			return ItemToEdit->GetName();
		}

		return FText::GetEmpty();
	}

	float OnReadValue() const
	{
		return ItemToEdit->GetNumber();
	}

	void OnWriteValue( float NewValue )
	{
		ItemToEdit->SetNumber(NewValue);
	}

	/** A pointer to the data item that we visualize/edit */
	TSharedPtr<FTestData> ItemToEdit;
};

/**
 * An Item Editor used by tile view testing.
 * It visualizes a string and edits its contents.
 */
class SHorizontalListItemEditor : public STableRow< TSharedPtr<FTestData> >
{
public:
	
	SLATE_BEGIN_ARGS(SHorizontalListItemEditor)
		: _ItemToEdit()
		{}

		SLATE_EVENT(FOnCanAcceptDrop, OnCanAcceptDrop)
		SLATE_EVENT(FOnAcceptDrop, OnAcceptDrop)
		SLATE_EVENT(FOnDragDetected, OnDragDetected)
		SLATE_ARGUMENT(TSharedPtr<FTestData>, ItemToEdit)

	SLATE_END_ARGS()
	
	/**
	 * Construct the widget
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		STableRow<TSharedPtr<FTestData> >::ConstructInternal( STableRow::FArguments()
			.OnCanAcceptDrop(InArgs._OnCanAcceptDrop)
			.OnAcceptDrop(InArgs._OnAcceptDrop)
			.OnDragDetected(InArgs._OnDragDetected)
			.ShowSelection(true)
		, InOwnerTableView);

		ItemToEdit = InArgs._ItemToEdit;

		FSlateFontInfo ItemEditorFont = FCoreStyle::Get().GetFontStyle(TEXT("NormalFont"));

		this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage( FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder") )
			.BorderBackgroundColor(FLinearColor(1,1,1,0.45f))
			.Padding(this, &SHorizontalListItemEditor::GetVariableWidth)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4)
				[
					// Name
					SNew( STextBlock )
					.Font( ItemEditorFont )
					.Text( this, &SHorizontalListItemEditor::OnReadText )
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
				[
					// Number
					SNew(SSpinBox<float>)
					.MinValue(-5)
					.MaxValue(800)
					.Font( ItemEditorFont )
					.Value( this, &SHorizontalListItemEditor::OnReadValue )
					.OnValueChanged( this, &SHorizontalListItemEditor::OnWriteValue )
				]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				//TextField
				SNew( SEditableTextBox )
				.Font( ItemEditorFont )
				.Text( this, &SHorizontalListItemEditor::OnReadText )
				.OnTextChanged( this, &SHorizontalListItemEditor::OnTextChanged )
			]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
				[
					//TextBlock
					SNew( STextBlock ) .Font( ItemEditorFont ) .Text( this, &SHorizontalListItemEditor::OnReadText )
				]
			]
		];
	}
	
	void SpawnContextMenu( const FVector2D& SpawnLocation )
	{
		FSlateApplication::Get().PushMenu
		(
			SharedThis( this ),
			FWidgetPath(),
			GetPopupContent(),
			SpawnLocation,
			FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu )
		);
	}
	
private:
	FMargin GetVariableWidth() const
	{
		return FMargin(0, 0, this->ItemToEdit->GetNumber(), 0);
	}

	TSharedRef<SWidget> GetPopupContent()
	{		
		return SNew(STestMenu);
	}

	/** Modify the text when the widget changes it */
	void OnTextChanged( const FText& NewText )
	{
		if (ItemToEdit.IsValid())
		{
			ItemToEdit->SetName( NewText );
		}		
	}
	
	/** @return the text being edited by this widget */
	FText OnReadText() const
	{
		if ( ItemToEdit.IsValid() )
		{
			return ItemToEdit->GetName();
		}

		return FText::GetEmpty();
	}

	float OnReadValue() const
	{
		return ItemToEdit->GetNumber();
	}

	void OnWriteValue( float NewValue )
	{
		ItemToEdit->SetNumber(NewValue);
	}

	/** A pointer to the data item that we visualize/edit */
	TSharedPtr<FTestData> ItemToEdit;
};

/**
 * ListTesting is a test case for lists and trees.
 */
class STableViewTesting : public SCompoundWidget
{

/** A pointer to a SelectionMode */
typedef TSharedPtr<ESelectionMode::Type> ESelectionModePtr;

public:
	
	SLATE_BEGIN_ARGS( STableViewTesting ){}

	SLATE_END_ARGS()
	
	/**
	 * Create some lists and trees for testing purposes.
	 * 
	 * @param InArgs   Declaration from which to construct the widget
	 */
	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	void Construct( const FArguments& InArgs )
	{	
		TSharedRef<SScrollBar> ExternalScrollbar = SNew(SScrollBar);

		// This is a CompoundWidget, so we can assign arbitrary Widget content to the ChildSlot, and it
		// will become part of the widget hierarchy.
		this->ChildSlot
		[
			// Start by making a vertical box panel
			SNew( SVerticalBox )
			// The first slot is a row of information about the test case.
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding( 5.0f )
			[
				SNew( SHorizontalBox )
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock).Text( this, &STableViewTesting::GetNumGeneratedChildren )
				]
				+SHorizontalBox::Slot().AutoWidth() .Padding(20,0,0,0) .VAlign(VAlign_Center)
				[
					SNew(STextBlock).Text( LOCTEXT("TotalChildrenLabel", "Total children:") )
				]
				+SHorizontalBox::Slot().AutoWidth()
				[
					// We set how many data items we want in our test case.
					SNew(SNumericEntryBox<int32>)
					.Value( this, &STableViewTesting::GetNumTotalItems )
					.OnValueChanged( this, &STableViewTesting::NumItems_OnValueChanged )
				]
				+SHorizontalBox::Slot().AutoWidth()
				[
					// Press rebuild to clear out the old data items and create the new ones (however many are specified by SEditableTextBox)
					SNew(SButton)
					.Text( LOCTEXT("RebuildDataButtonLabel", "Rebuild Data") )
					.OnClicked( this, &STableViewTesting::Rebuild_OnClicked )
				]
				+SHorizontalBox::Slot().AutoWidth() .Padding(10, 0, 0, 0)
				[
					// We set how many data items we want in our test case.
					SNew(SNumericEntryBox<int32>)
					.Value( this, &STableViewTesting::GetScrollToIndex )
					.OnValueChanged( this, &STableViewTesting::ScrollToIndex_OnValueChanged )
				]
				+SHorizontalBox::Slot().AutoWidth()
				[
					// Press rebuild to clear out the old data items and create the new ones (however many are specified by SEditableTextBox)
					SNew(SButton)
					.Text(LOCTEXT("ScrollToItemButtonLabel", "Scroll to Item"))
					.OnClicked( this, &STableViewTesting::ScrollToIndex_OnClicked )
				]
				+SHorizontalBox::Slot() .FillWidth(1)
				[
					SNew(SSpacer)
				]
				+SHorizontalBox::Slot() .AutoWidth()
				[
					SNew(SCheckBox)
					.IsChecked(this, &STableViewTesting::IsScrollAnimChecked)
					.OnCheckStateChanged(this, &STableViewTesting::HandleScrollAnimCheckStateChanged)
				]
				+SHorizontalBox::Slot().AutoWidth() .Padding(2,0,0,0) .VAlign(VAlign_Center)
				[
					SNew(STextBlock).Text( LOCTEXT("AnimateScrollingLabel", "Animate Scrolling") )
				]
				+SHorizontalBox::Slot() .AutoWidth() .Padding(10, 0, 0, 0)
				[
					SNew(SCheckBox)
					.IsChecked(this, &STableViewTesting::IsFixedItemOffsetChecked)
					.OnCheckStateChanged(this, &STableViewTesting::HandleFixedItemOffsetCheckStateChanged)
				]
				+SHorizontalBox::Slot().AutoWidth()
				[
					SNew(SHorizontalBox)
					.IsEnabled(this, &STableViewTesting::GetIsFixedItemOffsetEnabled)
					+SHorizontalBox::Slot().AutoWidth() .VAlign(VAlign_Center)
					[
						SNew(STextBlock).Text(LOCTEXT("FixedLineScrollOffsetLabel", "Fixed Item Scroll Offset: "))
					]
					+SHorizontalBox::Slot().AutoWidth()
					[
						// We set how many data items we want in our test case.
						SNew(SNumericEntryBox<double>)
						.MinValue(0.)
						.MaxValue(0.5)
						.Value(this, &STableViewTesting::GetFixedItemOffsetValue)
						.OnValueChanged(this, &STableViewTesting::HandleFixedItemOffsetValueChanged)
					]
				]
				+SHorizontalBox::Slot() .FillWidth(1)
				[
					SNew(SSpacer)
				]
				+SHorizontalBox::Slot().AutoWidth()
				[
					SAssignNew(SelectionModeCombo, SComboBox< TSharedPtr<ESelectionMode::Type> >)
					.OptionsSource( &SelectionModes )
					.OnSelectionChanged( this, &STableViewTesting::OnSelectionModeChanged )
					.OnGenerateWidget( this, &STableViewTesting::GenerateSelectionModeMenuItem )
					[
						SNew(STextBlock)
						.Text( this, &STableViewTesting::GetSelectedModeText )
					]
				]
				+SHorizontalBox::Slot().AutoWidth()
				[
					SNew(SButton)
					.Text( LOCTEXT("RefreshButtonLabel", "Refresh!") )
					.OnClicked( this, &STableViewTesting::RequestRefresh )
				]
			]
			+SVerticalBox::Slot() .AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot() .AutoWidth() .VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text( LOCTEXT("ExpansionTestingLabel", "Expansion: ") )
				]
				+SHorizontalBox::Slot() .AutoWidth() .VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Text( LOCTEXT("RememberExpansionButton", "Remember") )
					.OnClicked( this, &STableViewTesting::RememberExpansion )
				]
				+SHorizontalBox::Slot() .AutoWidth() .VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Text( LOCTEXT("CollapseExpansionsButton", "Collapse All") )
					.OnClicked( this, &STableViewTesting::CollapseAll )
				]
				+SHorizontalBox::Slot() .AutoWidth() .VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Text( LOCTEXT("RestoreExpansionsButton", "Restore") )
					.OnClicked( this, &STableViewTesting::RestoreExpansion )
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1)
			[
				SNew(SBox)
				.HeightOverride(800.0f)
				[
					SNew(SSplitter)
					.Orientation(Orient_Vertical)
					+ SSplitter::Slot()
					.Value(.6f)
					[
						SNew( SSplitter )
						+ SSplitter::Slot()
						. Value(1)
						[
							SNew(SBorder)
							.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
							[
								SNew(SHorizontalBox)
								+SHorizontalBox::Slot()
								.AutoWidth()
								[
									ExternalScrollbar
								]
								+SHorizontalBox::Slot()
								.FillWidth(1)
								[
									// The list view being tested
									SAssignNew( ListBeingTested, SListView< TSharedPtr<FTestData> > )
									.ExternalScrollbar( ExternalScrollbar )
									// Tell the list view where to get its source data
									.ListItemsSource( &Items )
									// When the list view needs to generate a widget for some data item, use this method
									.OnGenerateRow( this, &STableViewTesting::OnGenerateWidgetForList )
									// What to put in the context menu
									.OnContextMenuOpening( this, &STableViewTesting::GetListContextMenu )
									// Single, multi or no selection
									.SelectionMode( this, &STableViewTesting::GetSelectionMode )
									.HeaderRow
									(
										SNew(SHeaderRow)
										+ SHeaderRow::Column("Name")
										[
											SNew(SBorder)
											.Padding(5)
											[
												SNew(STextBlock)
												.Text(LOCTEXT("TestNameColumn", "Name"))
											]
										]
										+ SHeaderRow::Column("Number") .DefaultLabel(LOCTEXT("TestNumberColumn", "Number"))
										+ SHeaderRow::Column("TextField") .DefaultLabel(LOCTEXT("TestTextFieldColumn", "Text Field"))
										+ SHeaderRow::Column("TextBlock") .DefaultLabel(LOCTEXT("TestTextBlockColumn", "Text Block"))
										+ SHeaderRow::Column("AddChild") .DefaultLabel(LOCTEXT("TestAddChildColumn", "Add Child"))
									)
								]
							]
						]
						+ SSplitter::Slot()
						. Value(1)
						[
							SNew(SBorder)
							.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
							[
								// The tile view being tested
								SAssignNew( TileViewBeingTested, STileView< TSharedPtr<FTestData> > )
								// Tile view items are this wide
								.ItemWidth(128)
								// Tile view items are this tall
								.ItemHeight(90)
								// Tell the tile view where to get its source data
								.ListItemsSource( &Items )
								// When the list view needs to generate a widget for some data item, use this method
								.OnGenerateTile( this, &STableViewTesting::OnGenerateWidgetForTileView )
								// What to put in the context menu
								.OnContextMenuOpening( this, &STableViewTesting::GetTileViewContextMenu )
								// Single, multi or no selection
								.SelectionMode( this, &STableViewTesting::GetSelectionMode )
							]
						]
						+ SSplitter::Slot()
						. Value(1)
						[
							SNew(SBorder)
							.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
							[
								// The TreeView being tested; mostly identical to ListView except for OnGetChildren
								SAssignNew( TreeBeingTested, STreeView< TSharedPtr<FTestData> > )
								.TreeItemsSource( &Items )
								.OnGenerateRow( this, &STableViewTesting::OnGenerateWidgetForTree )
								// Given some DataItem, this is how we find out if it has any children and what they are.
								.OnGetChildren( this, &STableViewTesting::OnGetChildrenForTree )
								// What to put in the context menu
								.OnContextMenuOpening( this, &STableViewTesting::GetTreeContextMenu )
								// Single, multi or no selection
								.SelectionMode( this, &STableViewTesting::GetSelectionMode )
								.WheelScrollMultiplier(60.f)
								.HeaderRow
								(
									SNew(SHeaderRow)
									+ SHeaderRow::Column("Name") .DefaultLabel(LOCTEXT("TestNameColumn","Name"))
									+ SHeaderRow::Column("Number") .DefaultLabel(LOCTEXT("TestNumberColumn", "Number"))
									+ SHeaderRow::Column("TextField") .DefaultLabel(LOCTEXT("TestTextFieldColumn", "Text Field"))
									+ SHeaderRow::Column("TextBlock") .DefaultLabel(LOCTEXT("TestTextBlockColumn", "Text Block"))
									+ SHeaderRow::Column("AddChild") .DefaultLabel(LOCTEXT("TestAddChildColumn", "Add Child"))
								)
							]
						]
					]
					+ SSplitter::Slot()
					.Value(.15f)
					[
						SNew(SBorder)
						.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
						[
							// The horizontally-oriented list view being tested
							SAssignNew(HorizontalListBeingTested, SListView< TSharedPtr<FTestData> > )
							// This is a horizontally scrolling list view
							.Orientation(Orient_Horizontal)
							// List view items are this tall
							//.ItemHeight(24)
							// Tell the list view where to get its source data
							.ListItemsSource( &Items )
							// When the list view needs to generate a widget for some data item, use this method
							.OnGenerateRow( this, &STableViewTesting::OnGenerateWidgetForHorizontalList )
							// What to put in the context menu
							.OnContextMenuOpening( this, &STableViewTesting::GetListContextMenu )
							// Single, multi or no selection
							.SelectionMode( this, &STableViewTesting::GetSelectionMode )
						]
					]
					+ SSplitter::Slot()
					.Value(.25f)
					[
						SNew(SBorder)
						.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
						[
							// The horizontally-oriented tile view being tested
							SAssignNew( HorizontalTileViewBeingTested, STileView< TSharedPtr<FTestData> > )
							// This is a horizontally scrolling tile view
							.Orientation(Orient_Horizontal)
							// Tile view items are this wide
							.ItemWidth(128)
							// Tile view items are this tall
							.ItemHeight(90)
							// Tell the tile view where to get its source data
							.ListItemsSource( &Items )
							// When the list view needs to generate a widget for some data item, use this method
							.OnGenerateTile( this, &STableViewTesting::OnGenerateWidgetForTileView )
							// What to put in the context menu
							.OnContextMenuOpening( this, &STableViewTesting::GetTileViewContextMenu )
							// Single, multi or no selection
							.SelectionMode( this, &STableViewTesting::GetSelectionMode )
						]
					]
				]
			]
		];
		
		// Populate the valid selection modes
		{
			SelectionModes.Add( MakeShareable( new ESelectionMode::Type(ESelectionMode::None) ) );
			SelectionModes.Add( MakeShareable( new ESelectionMode::Type(ESelectionMode::Single) ) );
			SelectionModes.Add( MakeShareable( new ESelectionMode::Type(ESelectionMode::SingleToggle) ) );
			SelectionModes.Add( MakeShareable( new ESelectionMode::Type(ESelectionMode::Multi) ) );
			CurSelectionMode = SelectionModes.Last();
			SelectionModeCombo->SetSelectedItem(CurSelectionMode);
		}

		// Rebuild all the items as if we clicked the Rebuild button	
		Rebuild_OnClicked();
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

	STableViewTesting()
	: TotalItems( 5000 )
	, ScrollToIndex( 0 )
	{
	}
	
protected:
	
	FReply OnDragDetected_Handler(const FGeometry&, const FPointerEvent&, TWeakPtr<FTestData> TestData)
	{
		return FReply::Handled().BeginDragDrop(FTableViewDragDrop::New(TestData.Pin().ToSharedRef()));
	}


	TOptional<EItemDropZone> OnCanAcceptDrop_Handler(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FTestData> TargetItem)
	{
		TSharedPtr<FTableViewDragDrop> DragDropOperation = DragDropEvent.GetOperationAs<FTableViewDragDrop>();
		if (DragDropOperation.IsValid())
		{
			return DropZone;
		}
		else
		{
			return TOptional<EItemDropZone>();
		}
	}


	FReply OnAcceptDrop_Handler(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FTestData> TargetItem)
	{
		TSharedPtr<FTableViewDragDrop> DragDropOperation = DragDropEvent.GetOperationAs<FTableViewDragDrop>();
		if (DragDropOperation.IsValid())
		{
			const TSharedRef<FTestData>& ItemBeingDragged = DragDropOperation->GetDraggedItem();
			FTestData::RemoveRecursive(Items, ItemBeingDragged);
			FTestData::InsertRecursive(Items, ItemBeingDragged, TargetItem.ToSharedRef(), DropZone);
			ListBeingTested->RequestListRefresh();
			HorizontalListBeingTested->RequestListRefresh();
			TreeBeingTested->RequestTreeRefresh();
			TileViewBeingTested->RequestListRefresh();
			HorizontalTileViewBeingTested->RequestListRefresh();

			return FReply::Handled();
		}
		else
		{
			return FReply::Unhandled();
		}		
	}

	TSharedPtr<SWidget> GetListContextMenu()
	{
		return 
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FCoreStyle::Get().GetBrush("Menu.Background"))
				.Padding( FMargin(5) )
				[
					SNew(STextBlock) .Text( LOCTEXT("ListContextMenuLabel", "List Context Menu" ) )
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STestMenu)
			];
			
	}

	TSharedPtr<SWidget> GetTileViewContextMenu()
	{
		return 
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FCoreStyle::Get().GetBrush("Menu.Background"))
				.Padding( FMargin(5) )
				[
					SNew(STextBlock) .Text( LOCTEXT("TileViewContextMenuLabel", "Tile view Context Menu" ) )
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STestMenu)
			];
			
	}

	TSharedPtr<SWidget> GetTreeContextMenu()
	{
		return
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FCoreStyle::Get().GetBrush("Menu.Background"))
			.Padding( FMargin(5) )
			[
				SNew(STextBlock) .Text( LOCTEXT("TreeContextMenuLabel", "Tree Context Menu" ) )
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STestMenu)
		];
	}

	FReply RememberExpansion()
	{
		StoredExpandedItems.Empty();
		TreeBeingTested->GetExpandedItems(StoredExpandedItems);
		return FReply::Handled();
	}
	TSet< TSharedPtr<FTestData> > StoredExpandedItems;

	FReply CollapseAll()
	{
		TreeBeingTested->ClearExpandedItems();
		TreeBeingTested->RequestTreeRefresh();
		return FReply::Handled();
	}

	FReply RestoreExpansion()
	{
		for ( const TSharedPtr<FTestData>& Item : StoredExpandedItems )
		{
			TreeBeingTested->SetItemExpansion( Item, true );
		}
		TreeBeingTested->RequestTreeRefresh();
		return FReply::Handled();
	}



	/** @returns How many widgets the TreeView generated to represent the data items*/
	FText GetNumGeneratedChildren() const
	{
		if (TreeBeingTested.IsValid())
		{
			FFormatNamedArguments Args;
			Args.Add( TEXT("NumberOfWidgets"), ListBeingTested->GetNumGeneratedChildren() );
			return FText::Format( LOCTEXT("NumberOfWidgetsLabel", "Number of widgets in list: {NumberOfWidgets}"), Args );
		}
		else
		{
			return FText::GetEmpty();
		}
	}
	
	/** The textbox representing the number of data items changed */
	void NumItems_OnValueChanged( int32 InNewValue )
	{
		TotalItems = InNewValue;
	}

	TOptional<int32> GetScrollToIndex() const
	{
		return ScrollToIndex;
	}
	
	void ScrollToIndex_OnValueChanged( int32 NewScrollToIndex )
	{
		ScrollToIndex = NewScrollToIndex;
	}
	
	FReply ScrollToIndex_OnClicked()
	{
		if (Items.IsValidIndex(ScrollToIndex))
		{
			ListBeingTested->RequestScrollIntoView(Items[ScrollToIndex]);
			ListBeingTested->SetSelection(Items[ScrollToIndex]);

			HorizontalListBeingTested->RequestScrollIntoView(Items[ScrollToIndex]);
			HorizontalListBeingTested->SetSelection(Items[ScrollToIndex]);

			TreeBeingTested->RequestScrollIntoView( Items[ ScrollToIndex ] );
			TreeBeingTested->SetSelection( Items[ ScrollToIndex ] );
			
			TileViewBeingTested->RequestScrollIntoView( Items[ ScrollToIndex ] );
			TileViewBeingTested->SetSelection( Items[ ScrollToIndex ] );
			
			HorizontalTileViewBeingTested->RequestScrollIntoView(Items[ScrollToIndex]);
			HorizontalTileViewBeingTested->SetSelection(Items[ScrollToIndex]);
		}

		return FReply::Handled();
	}
	
	/** Request that both Tree and List refresh themselves on the next tick */
	FReply RequestRefresh()
	{
		ListBeingTested->RequestListRefresh();
		HorizontalListBeingTested->RequestListRefresh();
		TileViewBeingTested->RequestListRefresh();
		HorizontalTileViewBeingTested->RequestListRefresh();
		TreeBeingTested->RequestTreeRefresh();

		return FReply::Handled();
	}

	/** The user clicked a button to rebuild the test data */
	FReply Rebuild_OnClicked()
	{
		if ( Items.Num() != TotalItems )
		{
			Items.Empty();
			for( int32 ItemIndex = 0; ItemIndex < TotalItems; ++ItemIndex )
			{
				TSharedRef<FTestData> NewItem = FTestData::Make( FText::Format( LOCTEXT( "TestWidget", "Text Wgt {0}"), FText::AsNumber( ItemIndex ) ) );
				Items.Add( NewItem );
				FTestData::GenerateChildren( NewItem, 20, 0 );
			}
			
			RequestRefresh();		
		}
		return FReply::Handled();
	}
	
	/** @return How many data items we want to be using.*/
	TOptional<int32> GetNumTotalItems() const
	{
		return TotalItems;
	}
	
	/** @return Given a data item return a new widget to represent it in the ListView */
	TSharedRef<ITableRow> OnGenerateWidgetForList( TSharedPtr<FTestData> InItem, const TSharedRef<STableViewBase>& OwnerTable )
	{
		return
			SNew( SItemEditor, OwnerTable )
			// Triggered when a user is attempting to drag, so initiate a DragDropOperation.
			.OnDragDetected(this, &STableViewTesting::OnDragDetected_Handler, TWeakPtr<FTestData>(InItem))

			// Given a hovered drop zone (above, onto, or below) respond with a zone where we would actually drop the item.
			// Respond with an un-set TOptional<EItemDropZone> if we cannot drop here at all.
			.OnCanAcceptDrop(this, &STableViewTesting::OnCanAcceptDrop_Handler)

			// Actually perform the drop into the given drop zone.
			.OnAcceptDrop(this, &STableViewTesting::OnAcceptDrop_Handler)

			.ItemToEdit( InItem );
	}

	/** @return Given a data item return a new widget to represent it in the ListView */
	TSharedRef<ITableRow> OnGenerateWidgetForHorizontalList(TSharedPtr<FTestData> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return
			SNew(SHorizontalListItemEditor, OwnerTable)
			.OnCanAcceptDrop(this, &STableViewTesting::OnCanAcceptDrop_Handler)
			.OnAcceptDrop(this, &STableViewTesting::OnAcceptDrop_Handler)
			.OnDragDetected(this, &STableViewTesting::OnDragDetected_Handler, TWeakPtr<FTestData>(InItem))
			.ItemToEdit(InItem);
	}

	// Tile view test

	/** @return Given a data item return a new widget to represent it in the ListView */
	TSharedRef<ITableRow> OnGenerateWidgetForTileView( TSharedPtr<FTestData> InItem, const TSharedRef<STableViewBase>& OwnerTable )
	{
		return
			SNew( STileItemEditor, OwnerTable )
			.OnCanAcceptDrop(this, &STableViewTesting::OnCanAcceptDrop_Handler)
			.OnAcceptDrop(this, &STableViewTesting::OnAcceptDrop_Handler)
			.OnDragDetected(this, &STableViewTesting::OnDragDetected_Handler, TWeakPtr<FTestData>(InItem))
			.ItemToEdit( InItem );
	}
	
	
	// Tree test
	
	/** @return A widget to represent a data item in the TreeView */
	TSharedRef<ITableRow> OnGenerateWidgetForTree( TSharedPtr<FTestData> InItem, const TSharedRef<STableViewBase>& OwnerTable )
	{
		return
			SNew( SItemEditor, OwnerTable )
			.OnCanAcceptDrop( this, &STableViewTesting::OnCanAcceptDrop_Handler )
			.OnAcceptDrop( this, &STableViewTesting::OnAcceptDrop_Handler )
			.OnDragDetected(this, &STableViewTesting::OnDragDetected_Handler, TWeakPtr<FTestData>(InItem))
			.ItemToEdit( InItem );
	}
	
	/** Given a data item populate the OutChildren array with the item's children. */
	void OnGetChildrenForTree( TSharedPtr< FTestData > InItem, TArray< TSharedPtr<FTestData> >& OutChildren )
	{
		InItem->GetChildren( OutChildren );
	}

	FText GetSelectedModeText() const
	{
		ESelectionModePtr Mode = SelectionModeCombo->GetSelectedItem();
		return (Mode.IsValid())
			? GetSelectedModeText(Mode)
			: FText::GetEmpty();
	}

	ECheckBoxState IsScrollAnimChecked() const
	{
		return bIsScrollAnimEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void HandleScrollAnimCheckStateChanged(ECheckBoxState NewState)
	{
		bIsScrollAnimEnabled = NewState == ECheckBoxState::Checked;
		
		ListBeingTested->SetIsScrollAnimationEnabled(bIsScrollAnimEnabled);
		HorizontalListBeingTested->SetIsScrollAnimationEnabled(bIsScrollAnimEnabled);
		TileViewBeingTested->SetIsScrollAnimationEnabled(bIsScrollAnimEnabled);
		HorizontalTileViewBeingTested->SetIsScrollAnimationEnabled(bIsScrollAnimEnabled);
		TreeBeingTested->SetIsScrollAnimationEnabled(bIsScrollAnimEnabled);
	}

	ECheckBoxState IsFixedItemOffsetChecked() const
	{
		return bIsFixedLineScrollOffsetEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void HandleFixedItemOffsetCheckStateChanged(ECheckBoxState NewState)
	{
		bIsFixedLineScrollOffsetEnabled = NewState == ECheckBoxState::Checked;
		UpdateFixedItemOffset();
	}

	bool GetIsFixedItemOffsetEnabled() const
	{
		return bIsFixedLineScrollOffsetEnabled;
	}

	TOptional<double> GetFixedItemOffsetValue() const
	{
		return FixedLineScrollOffset;
	}

	void HandleFixedItemOffsetValueChanged(double NewValue)
	{
		FixedLineScrollOffset = NewValue;
		UpdateFixedItemOffset();
	}

	void UpdateFixedItemOffset()
	{
		TOptional<double> FixedOffset = bIsFixedLineScrollOffsetEnabled ? FixedLineScrollOffset : TOptional<double>();

		ListBeingTested->SetFixedLineScrollOffset(FixedOffset);
		HorizontalListBeingTested->SetFixedLineScrollOffset(FixedOffset);
		TileViewBeingTested->SetFixedLineScrollOffset(FixedOffset);
		HorizontalTileViewBeingTested->SetFixedLineScrollOffset(FixedOffset);
		TreeBeingTested->SetFixedLineScrollOffset(FixedOffset);
	}

	void OnSelectionModeChanged( ESelectionModePtr InMode, ESelectInfo::Type )
	{
		if (InMode.IsValid())
	{
		CurSelectionMode = InMode;
	}
	}
	
	FText GetSelectedModeText (ESelectionModePtr InMode) const
	{
		FText EnumText;

		switch( *InMode )
		{ 
			default:
		case ESelectionMode::None:
			EnumText = LOCTEXT("ESelectionMode::None", "None");
			break;

		case ESelectionMode::Single: 
			EnumText = LOCTEXT("ESelectionMode::Single", "Single");
			break;

		case ESelectionMode::SingleToggle: 
			EnumText = LOCTEXT("ESelectionMode::SingleToggle", "SingleToggle");
			break;

		case ESelectionMode::Multi:
			EnumText = LOCTEXT("ESelectionMode::Multi", "Multi");
			break;
		}
		return EnumText;
	}

	TSharedRef<SWidget> GenerateSelectionModeMenuItem( ESelectionModePtr InMode )
	{
		return SNew(STextBlock) 
			.Text( GetSelectedModeText(InMode) );
	}

	ESelectionMode::Type GetSelectionMode() const
	{
		return *CurSelectionMode;
	}
	
	ESelectionModePtr GetSelected() const
	{
		return CurSelectionMode;
	}
	
	/** A pointer to the ListView being tested */
	TSharedPtr< SListView< TSharedPtr<FTestData> > > ListBeingTested;
	TSharedPtr< SListView< TSharedPtr<FTestData> > > HorizontalListBeingTested;
	/** A pointer to the TileView being tested */
	TSharedPtr< STileView< TSharedPtr<FTestData> > > TileViewBeingTested;
	TSharedPtr< STileView< TSharedPtr<FTestData> > > HorizontalTileViewBeingTested;
	/** A pointer to the TreeView being tested */
	TSharedPtr< STreeView< TSharedPtr<FTestData> > > TreeBeingTested;
	/** The data items being tested */
	TArray< TSharedPtr< FTestData > > Items;
	/** How many top-level data item we want to use in testing */
	int32 TotalItems;
	/** Index of item to which we should scroll and highlight when the user presses the Scroll To Item button */
	int32 ScrollToIndex;


	/** Current selection Mode */
	ESelectionModePtr CurSelectionMode;
	/** Combo box for the selection mode used by the list/tree being tested */
	TSharedPtr< SComboBox< ESelectionModePtr > > SelectionModeCombo;
	/** All available selection modes */
	TArray< ESelectionModePtr > SelectionModes;

	bool bIsScrollAnimEnabled = false;
	bool bIsFixedLineScrollOffsetEnabled = false;
	double FixedLineScrollOffset = 0.f;
};




TSharedRef<SWidget> MakeTableViewTesting()
{
	extern TOptional<FSlateRenderTransform> GetTestRenderTransform();
	extern FVector2D GetTestRenderTransformPivot();
	return
		SNew(STableViewTesting)
		.RenderTransform_Static(&GetTestRenderTransform)
		.RenderTransformPivot_Static(&GetTestRenderTransformPivot);
}


#undef LOCTEXT_NAMESPACE

#endif // #if !UE_BUILD_SHIPPING
