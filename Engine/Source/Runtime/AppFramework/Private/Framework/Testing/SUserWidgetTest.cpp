// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Testing/SUserWidgetTest.h"
#include "Widgets/Text/STextBlock.h"

#if !UE_BUILD_SHIPPING

class SUserWidgetExampleImpl
	: public SUserWidgetExample
{
public:

	void Construct( const FArguments& InArgs ) override
	{
		SUserWidget::Construct( SUserWidget::FArguments()
		[
			SNew(STextBlock)
				.Text( FText::Format( NSLOCTEXT("SlateTestSuite","UserWidgetExampleTitle"," Implemented in the .cpp : {0}"), InArgs._Title ) )
		]);
	}

	virtual void DoStuff() override
	{

	}
};


TSharedRef<SUserWidgetExample> SUserWidgetExample::New()
{
	return MakeShareable(new SUserWidgetExampleImpl()); 
}

#endif // #if !UE_BUILD_SHIPPING
