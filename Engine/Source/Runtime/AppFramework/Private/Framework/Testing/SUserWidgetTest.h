// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SUserWidget.h"

#if !UE_BUILD_SHIPPING


class SUserWidgetExample
	: public SUserWidget
{
	public:

	SLATE_USER_ARGS(SUserWidgetExample)
		: _Title()
	{ }
		SLATE_ARGUMENT(FText, Title)
	SLATE_END_ARGS()

	virtual void Construct( const FArguments& InArgs ) = 0;

	virtual void DoStuff() = 0;
};

#endif // #if !UE_BUILD_SHIPPING
