// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Testing/SWidgetGallery.h"
#include "Animation/CurveSequence.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Layout/WidgetPath.h"
#include "SlateOptMacros.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SFxWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SSpinningImage.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SHeader.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Text/SMultiLineEditableText.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Input/SComboBox.h"
#include "Framework/Docking/TabManager.h"

#if !UE_BUILD_SHIPPING

#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Testing/TestStyle.h"

#include "Framework/Testing/SUserWidgetTest.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/SVolumeControl.h"
#include "Widgets/Input/STextComboPopup.h"


#define LOCTEXT_NAMESPACE "WidgetGallery"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

/**
 * Implements a widget gallery.
 *
 * The widget gallery demonstrates the widgets available in the core of the Slate.
 * Update the PopulateGallery() method to add your new widgets.
 */
class SWidgetGallery
	: public SCompoundWidget
{
	// Enumerates radio button choices.
	enum ERadioChoice
	{
		Radio0,
		Radio1,
		Radio2,
	};

public:

	SLATE_BEGIN_ARGS( SWidgetGallery ) { }
	SLATE_END_ARGS()
		
public:

	/**
	 * Constructs the widget gallery.
	 *
	 * @param InArgs - Construction arguments.
	 */
	void Construct( const FArguments& InArgs )
	{
		PopupMethod = EPopupMethod::CreateNewWindow;

		// example of tab activation registration
		{
			IsActiveTabVisibility = EVisibility::Visible;
			FGlobalTabmanager::Get()->OnActiveTabChanged_Subscribe(FOnActiveTabChanged::FDelegate::CreateSP(this, &SWidgetGallery::HandleTabmanagerActiveTabChanged));
		}

		// test options for STextComboBox example
		{
			TextComboBoxSelectedItem = MakeShareable(new FString("Option i"));
			TextComboBoxOptions.Add(TextComboBoxSelectedItem);
			TextComboBoxOptions.Add(MakeShareable(new FString("Option ii")));
			TextComboBoxOptions.Add(MakeShareable(new FString("Option iii")));

			ProgressCurve = FCurveSequence(0.0f,15.0f);
			ProgressCurve.Play(this->AsShared());
		}

		TSharedPtr<SMenuAnchor> StacklessAnchor1, StacklessAnchor2;
		
		ChildSlot
		[
			SNew(SBorder)
			.BorderImage( FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder") )
			[
				SNew(SScrollBox)

				+ SScrollBox::Slot()
					.Padding(5.0f)
					[
						SNew(SGridPanel)
							.FillColumn(0, 0.5f)
							.FillColumn(1, 0.5f)

						// SBorder
						+ SGridPanel::Slot(0, 0)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SBorderLabel", "SBorder"))
							]

						+ SGridPanel::Slot(1, 0)
							.Padding(0.0f, 5.0f)
							[
								SNew(SBorder)
									[
										SNew(SSpacer)
											.Size(FVector2D(100.0f, 50.0f))
									]	
							]

						// SBreadcrumbTrailLabel
						+ SGridPanel::Slot(0, 1)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SBreadcrumbTrailLabel", "SBreadcrumbTrail"))
							]

						+ SGridPanel::Slot(1, 1)
							.Padding(0.0f, 5.0f)
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
									.FillWidth(1.0f)
									[
										SAssignNew(BreadcrumbTrail, SBreadcrumbTrail<int32>)

									]

								+ SHorizontalBox::Slot()
									.AutoWidth()
									[
										SNew(SButton)
											.Text(LOCTEXT("AddBreadCrumbLabel", "Add"))
											.HAlign(HAlign_Center)
											.VAlign(VAlign_Center)
											.OnClicked(this, &SWidgetGallery::HandleBreadcrumbTrailAddButtonClicked)
									]
							]

						// SButton
						+ SGridPanel::Slot(0, 2)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SButtonLabel", "SButton"))
							]

						+ SGridPanel::Slot(1, 2)
							.HAlign(HAlign_Left)
							.Padding(0.0f, 5.0f)
							[
								SNew(SButton)
									.Text(LOCTEXT("ButtonExampleLabel", "Button"))
							]

						// SButton (no content)
						+ SGridPanel::Slot(0, 3)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SButtonExampleLabel", "SButton (no content)"))
							]

						+ SGridPanel::Slot(1, 3)
							.HAlign(HAlign_Left)
							.Padding(0.0f, 5.0f)
							[
								SNew(SButton)
							]

						// SCheckBox
						+ SGridPanel::Slot(0, 4)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SCheckBoxLabel", "SCheckBox"))
							]

						+ SGridPanel::Slot(1, 4)
							.HAlign(HAlign_Left)
							.Padding(0.0f, 5.0f)
							[
								SNew(SVerticalBox)

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										CreateCheckBox(LOCTEXT("SCheckBoxItemLabel01", "Option 1"), &CheckBox1Choice)
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										CreateCheckBox(LOCTEXT("SCheckBoxItemLabel02", "Option 2"), &CheckBox2Choice)
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										CreateCheckBox(LOCTEXT("SCheckBoxItemLabel03", "Option 3"), &CheckBox3Choice)
									]
							]

						// SCheckBox (as radio button)
						+ SGridPanel::Slot(0, 5)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SRadioButtonLabel", "SCheckBox (as radio buttons)"))
							]

						+ SGridPanel::Slot(1, 5)
							.HAlign(HAlign_Left)
							.Padding(0.0f, 5.0f)
							[
								SNew(SVerticalBox)

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										CreateRadioButton(LOCTEXT("SRadioButtonItemLabel01", "Option 1"), Radio0)
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										CreateRadioButton(LOCTEXT("SRadioButtonItemLabel02", "Option 2"), Radio1)
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										CreateRadioButton(LOCTEXT("SRadioButtonItemLabel03", "Option 3"), Radio2)
									]
							]

						// SCircularThrobber
						+ SGridPanel::Slot(0, 6)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SCircularThrobberLabel", "SCircularThrobber"))
							]

						+ SGridPanel::Slot(1, 6)
							.HAlign(HAlign_Left)
							.Padding(0.0f, 5.0f)
							[
								SNew(SCircularThrobber)
							]

						// SColorBlock
						+ SGridPanel::Slot(0, 7)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SColorBlockLabel", "SColorBlock"))
							]

						+ SGridPanel::Slot(1, 7)
							.Padding(0.0f, 5.0f)
							[
								SNew(SColorBlock)
									.Color(FLinearColor(1.0f, 0.0f, 0.0f))
							]

						// SComboBox
						+ SGridPanel::Slot(0, 8)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SComboBoxLabel", "SComboBox"))
							]

						+ SGridPanel::Slot(1, 8)
							.HAlign(HAlign_Left)
							.Padding(0.0f, 5.0f)
							[
								SNew(SVerticalBox)

								+ SVerticalBox::Slot()
								[
									SNew(SCheckBox)
									.IsChecked(this, &SWidgetGallery::IsCreateNewWindowChecked)
									.OnCheckStateChanged( this, &SWidgetGallery::CreateNewWindowToggled )
									[
										SNew(STextBlock).Text(LOCTEXT("CreateNewWindowCheckbox", "Create new window for popups."))
									]
								]

								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									// selector combo box
									SAssignNew(SelectorComboBox, SComboBox<TSharedPtr<FString> >)
										.OptionsSource(&SelectorComboBoxOptions)
										.OnSelectionChanged(this, &SWidgetGallery::HandleSelectorComboBoxSelectionChanged)
										.OnGenerateWidget(this, &SWidgetGallery::HandleComboBoxGenerateWidget)
										[
											SNew(STextBlock)
												.Text(this, &SWidgetGallery::HandleSelectorComboBoxText)
										]
								]

								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									// second combo box
									SAssignNew(SecondComboBox, SComboBox<TSharedPtr<FString> >)
										.OptionsSource(&SecondComboBoxOptions)
										.OnSelectionChanged(this, &SWidgetGallery::HandleSecondComboBoxSelectionChanged)
										.OnGenerateWidget(this, &SWidgetGallery::HandleComboBoxGenerateWidget)
										[
											SNew(STextBlock)
												.Text(this, &SWidgetGallery::HandleSecondComboBoxText)
										]
								]
							]

						// SComboButton
						+ SGridPanel::Slot(0, 9)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SComboButtonLabel", "SComboButton"))
							]

						+ SGridPanel::Slot(1, 9)
							.HAlign(HAlign_Left)
							.Padding(0.0f, 5.0f)
							[
								SNew(SFxWidget).IgnoreClipping(false)
								[
									SNew(SComboButton)
									.ButtonContent()
									[
										SNew(STextBlock)
											.Text(LOCTEXT("ComboButtonLabel", "Combo Button"))
									]
									.MenuContent()
									[
										SNew(SVerticalBox)

										+ SVerticalBox::Slot()
											.AutoHeight()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("ComboButtonItemLabel01", "Combo\n     Button\n  menu\n       content supports"))
											]

										+ SVerticalBox::Slot()
											.AutoHeight()
											.HAlign(HAlign_Center)
											[
												SNew(SButton)
													[ 
														SNew(STextBlock)
															.Text(LOCTEXT("ComboButtonItemLabel02", "arbitrary"))
													]
											]

										+ SVerticalBox::Slot()
											.AutoHeight()
											[
												SNew(STextBlock)
													.Text(LOCTEXT("ComboButtonItemLabel03", "widgets"))
											]
									]
								]
							]

						// SEditableText
						+ SGridPanel::Slot(0, 10)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SEditableTextLabel", "SEditableText"))
							]

						+ SGridPanel::Slot(1, 10)
							.HAlign(HAlign_Left)
							.Padding(0.0f, 5.0f)
							[
								SNew(SEditableText)
									.HintText(LOCTEXT("SEditableTextHint", "This is editable text"))
							]

						// SEditableTextBox
						+ SGridPanel::Slot(0, 11)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SEditableTextBoxLabel", "SEditableTextBox"))
							]

						+ SGridPanel::Slot(1, 11)
							.HAlign(HAlign_Left)
							.Padding(0.0f, 5.0f)
							[
								SNew(SEditableTextBox)
									.HintText(LOCTEXT("SEditableTextBoxHint", "This is an editable text box"))
							]

						// SMultiLineEditableText
						+ SGridPanel::Slot(0, 12)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("SMultiLineEditableTextLabel", "SMultiLineEditableText"))
							]

						+ SGridPanel::Slot(1, 12)
							.HAlign(HAlign_Left)
							.Padding(0.0f, 5.0f)
							[
								SNew(SMultiLineEditableText)
								.HintText(LOCTEXT("SMultiLineEditableTextHint", "This is multi-line editable text"))
							]

						// SMultiLineEditableTextBox
						+ SGridPanel::Slot(0, 13)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("SMultiLineEditableTextBoxLabel", "SMultiLineEditableTextBox"))
							]

						+ SGridPanel::Slot(1, 13)
							.HAlign(HAlign_Left)
							.Padding(0.0f, 5.0f)
							[
								SNew(SMultiLineEditableTextBox)
								.HintText(LOCTEXT("SMultiLineEditableTextBoxHint", "This is a multi-line editable text box"))
							]

						// SHeader
						+ SGridPanel::Slot(0, 14)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SHeaderLabel", "SHeader"))
							]

						+ SGridPanel::Slot(1, 14)
							.Padding(0.0f, 5.0f)
							[
								SNew(SHeader)
									.Content()
									[
										SNew(STextBlock)
											.Text(LOCTEXT("HeaderContentLabel", "Header Content"))
									]
							]

						// SHyperlink
						+ SGridPanel::Slot(0, 15)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SHyperlinkLabel", "SHyperlink"))
							]

						+ SGridPanel::Slot(1, 15)
							.HAlign(HAlign_Left)
							.Padding(0.0f, 5.0f)
							[
								SNew(SHyperlink)
									.Text(LOCTEXT("SHyperlinkText", "Text to appear in the Hyperlink widget."))
			
							]

						// SImage
						+ SGridPanel::Slot(0, 16)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SImageLabel", "SImage"))
							]

						+ SGridPanel::Slot(1, 16)
							.Padding(0.0f, 5.0f)
							[
								SNew(SImage)
									.Image(FTestStyle::Get().GetBrush(TEXT("NewLevelBlank")))			
							]
	/*
						// SNumericEntryBox
						+ SGridPanel::Slot(0, 17)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SNumericEntryBoxLabel", "SNumericEntryBox"))
							]

						+ SGridPanel::Slot(1, 17)
							[
								New(SNumericEntryBox<float>)
							]
	*/
						// SProgressBar
						+ SGridPanel::Slot(0, 18)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SProgressBarLabel", "SProgressBar"))
							]

						+ SGridPanel::Slot(1, 18)
							.Padding(0.0f, 5.0f)
							[
								SNew(SVerticalBox)

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SProgressBar)
											.Percent(this, &SWidgetGallery::HandleProgressBarPercent)
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SProgressBar)
									]
							]
	/*
						// SRotatorInputBox
						+ SGridPanel::Slot(0, 19)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SRotatorInputBoxLabel", "SRotatorInputBox"))
							]

						+ SGridPanel::Slot(1, 19)
							.Padding(0.0f, 5.0f)
							[
								SNew(SRotatorInputBox)
									.Roll(0.5f)
									.Pitch(0.0f)
									.Yaw(1.0f)			
							]
	*/
						// SSearchBox
						+ SGridPanel::Slot(0, 20)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SSearchBoxLabel", "SSearchBox"))
							]

						+ SGridPanel::Slot(1, 20)
							.Padding(0.0f, 5.0f)
							[
								SNew(SVerticalBox)

								+ SVerticalBox::Slot()
									.AutoHeight()
									.HAlign(HAlign_Left)
									[
										SNew(SSearchBox)
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									.HAlign(HAlign_Left)
									.Padding(0.0f, 4.0f, 0.0f, 0.0f)
									[
										SNew(SSearchBox)
											.DelayChangeNotificationsWhileTyping(true)
											.OnSearch(this, &SWidgetGallery::HandleSearchBoxSearch)
									]			
							]

						// SSeparator
						+ SGridPanel::Slot(0, 21)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SSeparatorLabel", "SSeparator"))
							]

						+ SGridPanel::Slot(1, 21)
							.HAlign(HAlign_Left)
							.Padding(0.0f, 5.0f)
							[
								SNew(SBox)
									.HeightOverride(100.0f)
									.WidthOverride(150.0f)
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.VAlign(VAlign_Center)
											.FillWidth(0.75f)
											[
												SNew(SSeparator)
													.Orientation(Orient_Horizontal)
											]

										+ SHorizontalBox::Slot()
											.HAlign(HAlign_Center)
											.FillWidth(0.25f)
											[
												SNew(SSeparator)
													.Orientation(Orient_Vertical)
											]
									]			
							]

						// SSlider
						+ SGridPanel::Slot(0, 22)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SSliderLabel", "SSlider"))
							]

						+ SGridPanel::Slot(1, 22)
							.HAlign(HAlign_Left)
							.Padding(0.0f, 5.0f)
							[
								SNew(SBox)
									.HeightOverride(100.0f)
									.WidthOverride(150.0f)
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.VAlign(VAlign_Center)
											.FillWidth(0.75f)
											[
												SNew(SSlider)
													.Orientation(Orient_Horizontal)
													.Value(0.5f)
											]

										+ SHorizontalBox::Slot()
											.HAlign(HAlign_Center)
											.FillWidth(0.25f)
											[
												SNew(SSlider)
													.Orientation(Orient_Vertical)
													.Value(0.5f)
											]
									]			
							]

						// SSlider (no indentation)
						+ SGridPanel::Slot(0, 23)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SSliderNoIndentLabel", "SSlider (no indentation)"))
							]

						+ SGridPanel::Slot(1, 23)
							.Padding(0.0f, 5.0f)
							[
								SNew(SBox)
									.HeightOverride(100.0f)
									.WidthOverride(150.0f)
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.VAlign(VAlign_Center)
											.FillWidth(0.75f)
											[
												SNew(SSlider)
													.IndentHandle(false)
													.Orientation(Orient_Horizontal)
													.Value(0.5f)
											]

										+ SHorizontalBox::Slot()
											.HAlign(HAlign_Center)
											.FillWidth(0.25f)
											[
												SNew(SSlider)
													.IndentHandle(false)
													.Orientation(Orient_Vertical)
													.Value(0.5f)
											]
									]			
							]

						// SSpacer
						+ SGridPanel::Slot(0, 24)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SSpacerLabel", "SSpacer"))
							]

						+ SGridPanel::Slot(1, 24)
							.Padding(0.0f, 5.0f)
							[
								SNew(SSpacer)
									.Size(FVector2D(100, 100))		
							]

						// SSpinningImage
						+ SGridPanel::Slot(0, 25)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SSpinningImageLabel", "SSpinningImage"))
							]

						+ SGridPanel::Slot(1, 25)
							.HAlign(HAlign_Left)
							.Padding(0.0f, 5.0f)
							[
								SNew(SSpinningImage)
									.Image(FTestStyle::Get().GetBrush("TestRotation16px"))
							]

						// SSpinBox
						+ SGridPanel::Slot(0, 26)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SSpinBoxLabel", "SSpinBox"))
							]

						+ SGridPanel::Slot(1, 26)
							.HAlign(HAlign_Left)
							.Padding(0.0f, 5.0f)
							[
								SNew(SSpinBox<float>)
									.MinValue(-1000.0f)
									.MaxValue(1000.0f)
									.MinSliderValue(TAttribute< TOptional<float> >(-500.0f))
									.MaxSliderValue(TAttribute< TOptional<float> >(500.0f))
									.Delta(0.5f)
							]

						// STextBlock
						+ SGridPanel::Slot(0, 27)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("STextBlockLabel", "STextBlock"))
							]

						+ SGridPanel::Slot(1, 27)
							.Padding(0.0f, 5.0f)
							[
								SNew(SVerticalBox)
							
								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(STextBlock)
										.Text(LOCTEXT("STextBlockLeftExampleLabel", "This is a left-justified text block"))
										.Justification(ETextJustify::Left)
								]

								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(STextBlock)
										.Text(LOCTEXT("STextBlockCenterExampleLabel", "This is a center-justified text block"))
										.Justification(ETextJustify::Center)
								]

								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(STextBlock)
										.Text(LOCTEXT("STextBlockRightExampleLabel", "This is a right-justified text block"))
										.Justification(ETextJustify::Right)
								]

								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(STextBlock)
										.TextStyle(FCoreStyle::Get(), "NormalUnderlinedText")
										.Text(LOCTEXT("STextBlockUnderlineLabel", "This is text is underlined using a line highlighter!"))
								]
							]

						// STextComboBox
						+ SGridPanel::Slot(0, 28)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("STextComboBoxLabel", "STextComboBox"))
							]

						+ SGridPanel::Slot(1, 28)
							.Padding(0.0f, 5.0f)
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							[
								SAssignNew(TextComboBox, STextComboBox)
									.OptionsSource(&TextComboBoxOptions)
									.OnSelectionChanged(this, &SWidgetGallery::HandleTextComboBoxSelectionChanged)
									.OnGetTextLabelForItem(this, &SWidgetGallery::HandleTextComboBoxGetTextLabelForItem)
									.InitiallySelectedItem(TextComboBoxSelectedItem)			
							]

						// STextComboPopup
						+ SGridPanel::Slot(0, 29)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("STextComboPopupLabel", "STextComboPopup"))
							]

						+ SGridPanel::Slot(1, 29)
							.Padding(0.0f, 5.0f)
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							[
								SNew(SButton)
									.Text(LOCTEXT("ButtonTestLabel", "Test"))
									.OnClicked(this, &SWidgetGallery::HandleTextComboPopupClicked)
							]
		
						// SThrobber
						+ SGridPanel::Slot(0, 30)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SThrobberLabel", "SThrobber"))
							]

						+ SGridPanel::Slot(1, 30)
							.Padding(0.0f, 5.0f)
							[
								SNew(SVerticalBox)

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SThrobber)
											.Animate(SThrobber::Horizontal)
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SThrobber)
											.Animate(SThrobber::Opacity)
									]

								+ SVerticalBox::Slot()
									.AutoHeight()
									[
										SNew(SThrobber)
											.Animate(SThrobber::VerticalAndOpacity) .NumPieces(5)
									]
							]
	/*
						// SVectorInputBox
						+ SGridPanel::Slot(0, 31)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SVectorInputBoxLabel", "SVectorInputBox"))
							]

						+ SGridPanel::Slot(1, 31)
							.Padding(0.0f, 5.0f)
							[
								SNew(SVectorInputBox)
									.X(-5)
									.Y(0)
									.Z(5)			
							]
	*/
						// SVolumeControl
						+ SGridPanel::Slot(0, 32)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("SVolumeControlLabel", "SVolumeControl"))
							]

						+ SGridPanel::Slot(1, 32)
							.HAlign(HAlign_Left)
							.Padding(0.0f, 5.0f)
							[
								SNew(SBox)
									.WidthOverride(150.0f)
									[
										SNew(SVolumeControl)
											.Volume(0.6f)
									]
							]

						+ SGridPanel::Slot(0, 33)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("SUserWidgetLabel", "SUserWidgetExample"))
						]

						+ SGridPanel::Slot(1, 33)
						.HAlign(HAlign_Left)
						.Padding(0.0f, 5.0f)
						[
							SNew(SUserWidgetExample)
							.Title( LOCTEXT("SUserWidgetTest", "UserWidgetTest.cpp") )
						]

						// Menu anchors that don't use the menu stack
						+ SGridPanel::Slot(0, 34)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("StacklessAnchorLabel", "SMenuAnchor with UseApplicationMenuStack=false"))
						]
						+ SGridPanel::Slot(1, 34)
						.HAlign(HAlign_Left)
						.Padding(0.0f, 5.0f)
						[
							SNew(SButton)
							.OnClicked(this, &SWidgetGallery::OnClickStacklessAnchor, StacklessAnchor1)
							[
								SAssignNew(StacklessAnchor1, SMenuAnchor)
								.Placement(MenuPlacement_BelowAnchor)
								.Method(EPopupMethod::UseCurrentWindow)
								.UseApplicationMenuStack(false)
								.OnGetMenuContent(this, &SWidgetGallery::MakeAnchorMenuContent, LOCTEXT("StacklessContentLabel1", "Stackless Content 1"))
								[
									SNew(STextBlock)
									.Text(LOCTEXT("ClickMeLabel", "Click Me!"))
								]
							]
						]

						// Menu anchors that don't use the menu stack
						+ SGridPanel::Slot(0, 35)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("StacklessAnchorLabel", "SMenuAnchor with UseApplicationMenuStack=false"))
						]
						+ SGridPanel::Slot(1, 35)
						.HAlign(HAlign_Left)
						.Padding(0.0f, 5.0f)
						[
							SNew(SButton)
							.OnClicked(this, &SWidgetGallery::OnClickStacklessAnchor, StacklessAnchor2)
							[
								SAssignNew(StacklessAnchor2, SMenuAnchor)
								.Placement(MenuPlacement_BelowAnchor)
								.Method(EPopupMethod::CreateNewWindow)
								.UseApplicationMenuStack(false)
								.OnGetMenuContent(this, &SWidgetGallery::MakeAnchorMenuContent, LOCTEXT("StacklessContentLabel2", "Stackless Content 2"))
								[
									SNew(STextBlock)
									.Text(LOCTEXT("ClickMeLabel", "Click Me!"))
								]
							]
						]
					]
			]
		];

		// initialize SBreadcrumbTrail
		{
			BreadcrumbTrail->PushCrumb(LOCTEXT("PlaceholderRootBreadcrumb", "RootCrumb"), 0);
			BreadcrumbTrail->PushCrumb(LOCTEXT("PlaceholderBreadcrumb", "SomeCrumb"), 549);
			BreadcrumbTrail->PushCrumb(LOCTEXT("PlaceholderBreadcrumb", "SomeCrumb"), 33);
		}

		// initialize SCheckBox
		{
			CheckBox1Choice = false;
			CheckBox2Choice = true;
			CheckBox3Choice = false;

		}

		// initialize SCheckBox (as radio button)
		{
			RadioChoice = Radio0;
		}

		// initialize SComboBox
		{
			TSharedPtr<FString> SelectedItem = MakeShareable(new FString( "Options List A"));
			SelectorComboBoxOptions.Add(SelectedItem );
			SelectorComboBoxOptions.Add(MakeShareable(new FString("Options List B")));
			SelectorComboBoxSelectedItem = SelectedItem;

			SelectorComboBox->RefreshOptions();
			SelectorComboBox->SetSelectedItem(SelectedItem);

			SwitchSecondComboToOptionSetA();
		}
	}


protected:

	/**
	 * Creates a check box widget.
	 *
	 * @param CheckBoxText - The check box's text.
	 * @param CheckBoxChoice - The choice payload.
	 */
	TSharedRef<SWidget> CreateCheckBox( const FText& CheckBoxText, bool* CheckBoxChoice )
	{
		return SNew(SCheckBox)
			.IsChecked(this, &SWidgetGallery::HandleCheckBoxIsChecked, CheckBoxChoice)
			.OnCheckStateChanged(this, &SWidgetGallery::HandleCheckBoxCheckedStateChanged, CheckBoxChoice)
			[
				SNew(STextBlock)
					.Text(CheckBoxText)
			];
	}

	/**
	 * Creates a radio button widget.
	 *
	 * @param RadioText - The radio button's text.
	 * @param RadioButtonChoice - The choice payload.
	 */
	TSharedRef<SWidget> CreateRadioButton( const FText& RadioText, ERadioChoice RadioButtonChoice )
	{
		return SNew(SCheckBox)
			.Style(FCoreStyle::Get(), "RadioButton")
			.IsChecked(this, &SWidgetGallery::HandleRadioButtonIsChecked, RadioButtonChoice)
			.OnCheckStateChanged(this, &SWidgetGallery::HandleRadioButtonCheckStateChanged, RadioButtonChoice)
			[
				SNew(STextBlock)
					.Text(RadioText)
			];
	}

	/**
	 * Changes the options of the second combo box in the SComboBox to the first set.
	 *
	 * @see SwitchSecondComboToOptionSetB
	 */
	void SwitchSecondComboToOptionSetA( )
	{
		SecondComboBoxOptions.Empty();

		for (int32 ItemIndex=0; ItemIndex < 500; ++ItemIndex)
		{
			SecondComboBoxOptions.Add(MakeShareable(new FString(FString::Printf(TEXT("Item A %3d"), ItemIndex))));
		}

		SecondComboBox->RefreshOptions();
		SecondComboBox->SetSelectedItem(SecondComboBoxOptions[0]);
	}
	
	/**
	 * Changes the options of the second combo box in the SComboBox to the second set.
	 *
	 * @see SwitchSecondComboToOptionSetA
	 */
	void SwitchSecondComboToOptionSetB( )
	{
		SecondComboBoxOptions.Empty();
		SecondComboBoxOptions.Add(MakeShareable(new FString("Item B One")));
		SecondComboBoxOptions.Add(MakeShareable(new FString("Item B Two")));
		SecondComboBoxOptions.Add(MakeShareable(new FString("Item B Three")));
		SecondComboBoxOptions.Add(MakeShareable(new FString("Item B Four")));
		SecondComboBoxOptions.Add(MakeShareable(new FString("Item B Five")));
		SecondComboBoxOptions.Add(MakeShareable(new FString("Item B Six")));
		SecondComboBoxOptions.Add(MakeShareable(new FString("Item B Seven")));
		SecondComboBoxOptions.Add(MakeShareable(new FString("Item B Eight")));
		SecondComboBoxOptions.Add(MakeShareable(new FString("Item B Nine")));
		SecondComboBoxOptions.Add(MakeShareable(new FString("Item B Ten")));
		SecondComboBoxOptions.Add(MakeShareable(new FString("Item B Eleven")));
		SecondComboBoxOptions.Add(MakeShareable(new FString("Item B Twelve")));
		SecondComboBoxOptions.Add(MakeShareable(new FString("Item B Thirteen")));
		SecondComboBoxOptions.Add(MakeShareable(new FString("Item B Fourteen")));
		
		SecondComboBox->RefreshOptions();
		SecondComboBox->SetSelectedItem(SecondComboBoxOptions[0]);
	}

private:

	EPopupMethod PopupMethod;

	ECheckBoxState IsCreateNewWindowChecked() const
	{
		return this->PopupMethod == EPopupMethod::CreateNewWindow ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void CreateNewWindowToggled(ECheckBoxState NewState)
	{
		this->PopupMethod = (PopupMethod == EPopupMethod::CreateNewWindow)
			? EPopupMethod::UseCurrentWindow
			: EPopupMethod::CreateNewWindow;
	}


	// Callback for clicking the Add button in the SBreadcrumbTrail example.
	FReply HandleBreadcrumbTrailAddButtonClicked( )
	{
		BreadcrumbTrail->PushCrumb(LOCTEXT("PlaceholderBreadcrumb02", "SomeNewCrumb"), 0);

		return FReply::Handled();
	}

	// Callback for changing the checked state of a check box.
	void HandleCheckBoxCheckedStateChanged( ECheckBoxState NewState, bool* CheckBoxThatChanged )
	{
		*CheckBoxThatChanged = (NewState == ECheckBoxState::Checked);
	}

	// Callback for determining whether a check box is checked.
	ECheckBoxState HandleCheckBoxIsChecked( bool* CheckBox ) const
	{
		return (*CheckBox)
			? ECheckBoxState::Checked
			: ECheckBoxState::Unchecked;
	}

	// Callback for generating a widget in the SComboBox example.
	TSharedRef<SWidget> HandleComboBoxGenerateWidget( TSharedPtr<FString> InItem )
	{
		return SNew(STextBlock)
			.Text(FText::FromString(*InItem));
	}

	// Callback for getting the percent value in the SProgressBar example.
	TOptional<float> HandleProgressBarPercent( ) const
	{
		// Show some marquee, some progress and some 100% filled state.
		const float Progress = ProgressCurve.GetLerp();
		const float MarqueeTimeFraction = 0.5f;

		return (Progress < MarqueeTimeFraction)
			? TOptional<float>()
			: (Progress - MarqueeTimeFraction)/(MarqueeTimeFraction * 0.75f);
	}

	// Callback for checking a radio button.
	void HandleRadioButtonCheckStateChanged( ECheckBoxState NewRadioState, ERadioChoice RadioThatChanged )
	{
		if (NewRadioState == ECheckBoxState::Checked)
		{
			RadioChoice = RadioThatChanged;
		}
	}

	// Callback for determining whether a radio button is checked.
	ECheckBoxState HandleRadioButtonIsChecked( ERadioChoice ButtonId ) const
	{
		return (RadioChoice == ButtonId)
			? ECheckBoxState::Checked
			: ECheckBoxState::Unchecked;
	}

	// Callback for searching in the SSearchBox example.
	void HandleSearchBoxSearch( SSearchBox::SearchDirection Direction )
	{
	}

	// Callback for changing the second combo box's selection in SComboBox example.
	void HandleSecondComboBoxSelectionChanged ( TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo )
	{
		ComboString = NewSelection;
	}

	// Callback for getting the text of the second combo box in the SComboBox example.
	FText HandleSecondComboBoxText() const
	{
		return ComboString.IsValid() ? FText::FromString(*ComboString) : FText::GetEmpty();
	}

	// Callback for changing the selector combo box's selection in SComboBox example.
	void HandleSelectorComboBoxSelectionChanged( TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo )
	{
		if (SelectorComboBoxOptions[0] == NewSelection)
		{
			SwitchSecondComboToOptionSetA();
		}
		else
		{
			SwitchSecondComboToOptionSetB();
		}

		SelectorComboBoxSelectedItem = NewSelection;
	}

	// Callback for getting the text of the selector combo box in the SComboBox example.
	FText HandleSelectorComboBoxText( ) const
	{
		return SelectorComboBoxSelectedItem.IsValid() ? FText::FromString(*SelectorComboBoxSelectedItem) : FText::GetEmpty();
	}

	// Callback for changing the active tab.
	void HandleTabmanagerActiveTabChanged( TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated )
	{
		IsActiveTabVisibility = (NewlyActivated.IsValid() && NewlyActivated->GetContent() == SharedThis(this))
			? EVisibility::Visible
			: EVisibility::Collapsed;
	}

	// Callback for testing the formatting STextComboBox items' display.
	FString HandleTextComboBoxGetTextLabelForItem (TSharedPtr<FString> StringItem)
	{
		return  FString::Printf(TEXT("> %s"), **StringItem);
	}

	// Callback for selection changes in the STextComboBox example.
	void HandleTextComboBoxSelectionChanged (TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
	{
	}
	
	// Callback for clicking the combo box in the STextComboBox example.
	FReply HandleTextComboPopupClicked( )
	{
		TArray<FString> TextOptions;
		TextOptions.Add(TEXT("Option 1"));
		TextOptions.Add(TEXT("Option 2"));

		FSlateApplication::Get().PushMenu(
			SharedThis(this),
			FWidgetPath(),
			SNew(STextComboPopup)
				.TextOptions(TextOptions)
				.OnTextChosen(this, &SWidgetGallery::HandleTextComboPopupTextChosen),
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
		);

		return FReply::Handled();
	}

	// Callback for choosing text in the STextComboPopup example.
	void HandleTextComboPopupTextChosen( const FString& ChosenText )
	{
		FSlateApplication::Get().DismissAllMenus();
	}

	virtual FPopupMethodReply OnQueryPopupMethod() const override
	{
		return FPopupMethodReply::UseMethod(PopupMethod);
	}

	TSharedRef<SWidget> MakeAnchorMenuContent(FText ContentLabel) const
	{
		return SNew(SBorder)
			.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.BorderBackgroundColor(FLinearColor(0.0f, 0.0f, 0.0f, 1.0f))
			.Padding(2)
			[
				SNew(SBorder)
				.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				.BorderBackgroundColor(FLinearColor(0.5f, 0.5f, 0.5f, 1.0f))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(50)
					[
						SNew(STextBlock)
						.Text(ContentLabel)
					]
				]
			];
	}

	FReply OnClickStacklessAnchor(TSharedPtr<SMenuAnchor> ClickedAnchor)
	{
		ClickedAnchor->SetIsOpen(!ClickedAnchor->IsOpen());
		return FReply::Handled();
	}

private:

	// Holds the bread crumb trail widget.
	TSharedPtr<SBreadcrumbTrail<int32> > BreadcrumbTrail;

	// Holds the checked state of the first check box in the SCheckBox example.
	bool CheckBox1Choice;

	// Holds the checked state of the second check box in the SCheckBox example.
	bool CheckBox2Choice;

	// Holds the checked state of the third check box in the SCheckBox example.
	bool CheckBox3Choice;

	// Holds the selected combo box item.
	TSharedPtr<FString> ComboString;

	EVisibility IsActiveTabVisibility;

	// Holds the curve sequence for the SProgressBar example.
	FCurveSequence ProgressCurve;

	// Holds the current choice in the SCheckBox (as radio button) example.
	ERadioChoice RadioChoice;

	// Holds the second combo box in the SComboBox example.
	TSharedPtr<SComboBox<TSharedPtr<FString> > > SecondComboBox;

	// Holds the options for the second combo box in the SComboBox example.
	TArray<TSharedPtr<FString> > SecondComboBoxOptions;

	// Holds the selector combo box in the SComboBox demo.
	TSharedPtr<SComboBox<TSharedPtr<FString> > > SelectorComboBox;

	// Holds the options for the selector combo box in the SComboBox example.
	TArray<TSharedPtr<FString> > SelectorComboBoxOptions;

	// Holds the selected text in the SComboBox example.
	TSharedPtr<FString>	SelectorComboBoxSelectedItem;

	// Holds the combo box in the STextComboBox example.
	TSharedPtr<STextComboBox> TextComboBox;

	// Holds the text options for the STextComboBox example.
	TArray<TSharedPtr<FString> > TextComboBoxOptions;

	// Holds the selected item of the text combo box in the STextComboBox example.
	TSharedPtr<FString> TextComboBoxSelectedItem;
};


/**
 * Creates a new widget gallery.
 *
 * @return The new gallery widget.
 */
TSharedRef<SWidget> MakeWidgetGallery()
{
	extern TOptional<FSlateRenderTransform> GetTestRenderTransform();
	extern FVector2D GetTestRenderTransformPivot();
	return
		SNew(SWidgetGallery)
		.RenderTransform_Static(&GetTestRenderTransform)
		.RenderTransformPivot_Static(&GetTestRenderTransformPivot);
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef LOCTEXT_NAMESPACE

#endif // #if !UE_BUILD_SHIPPING
