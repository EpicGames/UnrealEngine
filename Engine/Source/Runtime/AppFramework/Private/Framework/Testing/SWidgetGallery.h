// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"

#if !UE_BUILD_SHIPPING

/** @return a new Drag and Drop test widget */
TSharedRef<SWidget> MakeWidgetGallery();

#endif // #if !UE_BUILD_SHIPPING
