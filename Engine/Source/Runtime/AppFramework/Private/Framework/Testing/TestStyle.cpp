// Copyright Epic Games, Inc. All Rights Reserved.

#include "Framework/Testing/TestStyle.h"
#include "Brushes/SlateBorderBrush.h"
#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Application/SlateWindowHelper.h"

#if !UE_BUILD_SHIPPING

#define RootToContentDir Style->RootToContentDir

TSharedPtr< ISlateStyle > FTestStyle::Instance = nullptr;


void FTestStyle::ResetToDefault()
{
	SetStyle( FTestStyle::Create() );
}

void FTestStyle::SetStyle( const TSharedRef< ISlateStyle >& NewStyle )
{
	if ( Instance.IsValid() )
	{
		FSlateStyleRegistry::UnRegisterSlateStyle( *Instance.Get() );
	}

	Instance = NewStyle;

	if ( Instance.IsValid() )
	{
		FSlateStyleRegistry::RegisterSlateStyle( *Instance.Get() );
	}
	else
	{
		ResetToDefault();
	}
}

TSharedRef< ISlateStyle > FTestStyle::Create()
{
	TSharedRef< FSlateStyleSet > Style = MakeShareable( new FSlateStyleSet("TestStyle") );
	Style->SetContentRoot( FPaths::EngineContentDir() / TEXT("Slate") );

	// Note, these sizes are in Slate Units.
	// Slate Units do NOT have to map to pixels.
	const FVector2D Icon5x16(5.0f, 16.0f);
	const FVector2D Icon8x4(8.0f, 4.0f);
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon10x10(10.0f, 10.0f);
	const FVector2D Icon12x12(12.0f, 12.0f);
	const FVector2D Icon12x16(12.0f, 16.0f);
	const FVector2D Icon14x14(14.0f, 14.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon22x22(22.0f, 22.0f);
	const FVector2D Icon24x24(24.0f, 24.0f);
	const FVector2D Icon25x25(25.0f, 25.0f);
	const FVector2D Icon32x32(32.0f, 32.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);
	const FVector2D Icon36x24(36.0f, 24.0f);
	const FVector2D Icon128x128(128.0f, 128.0f);

	// Normal Text
	const FTextBlockStyle NormalText = FTextBlockStyle()
		.SetFont( DEFAULT_FONT("Regular", 9) )
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetShadowOffset(FVector2D::ZeroVector)
		.SetShadowColorAndOpacity(FLinearColor::Black)
		.SetHighlightColor( FLinearColor( 0.02f, 0.3f, 0.0f ) )
		.SetHighlightShape( BOX_BRUSH("Common/TextBlockHighlightShape", FMargin(3.f/8.f) ) );

	Style->Set( "TestRotation40px", new IMAGE_BRUSH( "Testing/TestRotation", Icon40x40 ) );
	Style->Set( "TestRotation20px", new IMAGE_BRUSH( "Testing/TestRotation", Icon20x20 ) );
	Style->Set( "TestRotation16px", new IMAGE_BRUSH( "Testing/TestRotation", Icon16x16 ) );

	Style->Set( "GammaReference", new IMAGE_BRUSH( "Testing/GammaReference", FVector2D(256,128) ) );

	Style->Set( "UEIcon", new IMAGE_BRUSH_SVG( "Starship/Common/unreal", FVector2D(50,50) ) );

	Style->Set( "NewLevelBlank", new IMAGE_BRUSH( "Testing/NewLevelBlank", FVector2D(256,256) ) );

	Style->Set("White", new BOX_BRUSH("Testing/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), FLinearColor::White));
	Style->Set("Gray", new BOX_BRUSH("Testing/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), FLinearColor::Gray));
	Style->Set("Black", new BOX_BRUSH("Testing/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), FLinearColor::Black));

	// Testing images in the multi box test
	{
		Style->Set( "MultiBoxTest.FirstCommandInfo", new IMAGE_BRUSH( "Testing/BrushWireframe", Icon16x16 ) );
		Style->Set( "MultiBoxTest.SecondCommandInfo", new IMAGE_BRUSH( "Testing/Wireframe", Icon16x16 ) );
		Style->Set( "MultiBoxTest.ThirdCommandInfo", new IMAGE_BRUSH( "Testing/Unlit", Icon16x16 ) );
		Style->Set( "MultiBoxTest.FourthCommandInfo", new IMAGE_BRUSH( "Testing/Lit", Icon16x16 ) );
	}

	// RichText Test Showcase
	{
		Style->Set( "RichText.Background", new BOX_BRUSH( "Testing/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(0), FLinearColor(FColor(0xffeff3f3))) );

		Style->Set( "RichText.RoundedBackground", new BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, FLinearColor(FColor(0xffeff3f3)) ) );

		const FTextBlockStyle NormalRichTextStyle = FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT("Regular", 14) )
			.SetColorAndOpacity(FLinearColor(FColor( 0xff2c3e50 )));

		Style->Set( "RichText.Text", NormalRichTextStyle);
		Style->Set( "RichText.TextHighlight", FTextBlockStyle(NormalRichTextStyle)
			.SetColorAndOpacity(FLinearColor(FColor( 0xff19bc9c )))
			);

		Style->Set( "RichText.Text.Fancy", FTextBlockStyle(NormalRichTextStyle)
			.SetFont( DEFAULT_FONT("BlackItalic", 14) )
			.SetColorAndOpacity(FLinearColor(FColor( 0xff19bc9c )))
		);

		Style->Set( "RichText.Header", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT("Bold", 20) )
			.SetColorAndOpacity(FLinearColor(FColor( 0xff2c3e50 )))
		);

		const FButtonStyle DarkHyperlinkButton = FButtonStyle()
			.SetNormal ( BORDER_BRUSH( "Old/HyperlinkDotted", FMargin(0,0,0,3/16.0f), FLinearColor(FColor( 0xff2c3e50 )) ) )
			.SetPressed( FSlateNoResource() )
			.SetHovered( BORDER_BRUSH( "Old/HyperlinkUnderline", FMargin(0,0,0,3/16.0f), FLinearColor(FColor( 0xff2c3e50 )) ) );

		const FHyperlinkStyle DarkHyperlink = FHyperlinkStyle()
			.SetUnderlineStyle(DarkHyperlinkButton)
			.SetTextStyle(NormalRichTextStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set( "RichText.Hyperlink", DarkHyperlink );

		Style->Set( "RichText.ImageDecorator", FInlineTextImageStyle()
			.SetImage( IMAGE_BRUSH( "Testing/DefaultPawn_16px", Icon16x16 ) )
			.SetBaseline( 0 )
		);

		/*	Style->Set( "Test.SquiggleArrow", FInlineTextImageStyle()
		.SetImage( IMAGE_BRUSH( "Common/TinyChalkArrow", FVector2D( 71, 20 ) ) )
		.SetBaseline( -8 )
		);*/

		//Tagline
		{
			Style->Set("RichText.Tagline.Background", new BOX_BRUSH("Testing/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), FLinearColor(FColor(0xffdbe4e4))));
			Style->Set("RichText.Tagline.DarkBackground", new BOX_BRUSH("Testing/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), FLinearColor(0.55423f, 0.60548f, 0.60548f)));
			Style->Set( "RichText.Tagline.Text", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT("Bold", 24) )
				.SetColorAndOpacity(FLinearColor(FColor( 0xff2c3e50 )))
			);

			Style->Set( "RichText.Tagline.TextHighlight", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT("Bold", 24) )
				.SetColorAndOpacity(FLinearColor(FColor( 0xff19bc9c )))
			);

			Style->Set( "RichText.Tagline.SubtleText", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "Regular", 16 ) )
				.SetColorAndOpacity(FLinearColor(FColor( 0xff2c3e50 )))
			);

			Style->Set( "RichText.Tagline.SubtleTextHighlight", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "Regular", 16 ) )
				.SetColorAndOpacity(FLinearColor(FColor( 0xff19bc9c )))
			);
		}

		//Interactive example
		{
			Style->Set( "RichText.Interactive.Details.Background", new BOX_BRUSH( "Testing/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(0), FLinearColor(FColor(0xffdbe4e4))) );

			const FTextBlockStyle NormalInteractiveRichText = FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "Regular", 12 ) )
				.SetColorAndOpacity(FLinearColor(FColor( 0xff2c3e50 )));

			Style->Set( "RichText.Interactive.Text", NormalInteractiveRichText	);

			const FButtonStyle HyperlinkButton = FButtonStyle()
				.SetNormal ( BORDER_BRUSH( "Old/HyperlinkDotted", FMargin(0,0,0,3/16.0f), FLinearColor(FColor( 0xff2c3e50 )) ) )
				.SetPressed( FSlateNoResource() )
				.SetHovered( BORDER_BRUSH( "Old/HyperlinkUnderline", FMargin(0,0,0,3/16.0f), FLinearColor(FColor( 0xff2c3e50 )) ) );

			const FHyperlinkStyle Hyperlink = FHyperlinkStyle()
				.SetUnderlineStyle(HyperlinkButton)
				.SetTextStyle(NormalInteractiveRichText)
				.SetPadding(FMargin(0.0f));

			Style->Set( "RichText.Interactive.Text.Hyperlink", Hyperlink );

			// Dialogue
			{
				const FTextBlockStyle NormalInteractiveRichDialogueText = FTextBlockStyle(NormalText)
					.SetFont( DEFAULT_FONT( "Italic", 12 ) )
					.SetColorAndOpacity(FLinearColor(FColor( 0xff2c3e50 )));
				Style->Set( "RichText.Interactive.Text.Dialogue", NormalInteractiveRichDialogueText );

				Style->Set( "RichText.Interactive.Text.StrongDialogue", FTextBlockStyle(NormalInteractiveRichDialogueText)
					.SetFont( DEFAULT_FONT( "BoldItalic", 12 ) )
					.SetColorAndOpacity(FLinearColor(FColor( 0xff2c3e50 )))
					);

				const FButtonStyle DialogueHyperlinkButton = FButtonStyle()
					.SetNormal ( BORDER_BRUSH( "Old/HyperlinkDotted", FMargin(0,0,0,3/16.0f), FLinearColor(FColor( 0xff2c3e50 )) ) )
					.SetPressed( FSlateNoResource() )
					.SetHovered( BORDER_BRUSH( "Old/HyperlinkUnderline", FMargin(0,0,0,3/16.0f), FLinearColor(FColor( 0xff2c3e50 )) ) );

				const FHyperlinkStyle DialogueHyperlink = FHyperlinkStyle()
					.SetUnderlineStyle(DialogueHyperlinkButton)
					.SetTextStyle(NormalInteractiveRichDialogueText)
					.SetPadding(FMargin(0.0f));

				Style->Set( "RichText.Interactive.Text.DialogueHyperlink", DialogueHyperlink );
			}

			// Details
			{
				Style->Set( "RichText.Interactive.Details.Name.Text", FTextBlockStyle(NormalText)
					.SetFont( DEFAULT_FONT("Bold", 10) )
					.SetColorAndOpacity(FLinearColor(FColor( 0xff2c3e50 )))
					);

				Style->Set( "RichText.Interactive.Details.Value.Text", FTextBlockStyle(NormalText)
					.SetFont( DEFAULT_FONT( "Regular", 10) )
					.SetColorAndOpacity(FLinearColor(FColor( 0xff2c3e50 )))
					);

				/* Set images for various SCheckBox states ... */
				const FCheckBoxStyle CheckBoxStyle = FCheckBoxStyle()
					.SetCheckBoxType(ESlateCheckBoxType::CheckBox)
					.SetUncheckedImage( IMAGE_BRUSH( "Common/Checkbox", Icon16x16, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
					.SetUncheckedHoveredImage( IMAGE_BRUSH( "Common/Checkbox", Icon16x16, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
					.SetUncheckedPressedImage( IMAGE_BRUSH( "Common/Checkbox_Hovered", Icon16x16, FLinearColor( 0.75f, 0.75f, 0.75f ) ) )
					.SetCheckedImage( IMAGE_BRUSH( "Common/Checkbox_Checked_Hovered", Icon16x16, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
					.SetCheckedHoveredImage( IMAGE_BRUSH( "Common/Checkbox_Checked_Hovered", Icon16x16, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
					.SetCheckedPressedImage( IMAGE_BRUSH( "Common/Checkbox_Checked", Icon16x16, FLinearColor( 0.75f, 0.75f, 0.75f ) ) )
					.SetUndeterminedImage( IMAGE_BRUSH( "Common/Checkbox_Undetermined", Icon16x16, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
					.SetUndeterminedHoveredImage( IMAGE_BRUSH( "Common/Checkbox_Undetermined_Hovered", Icon16x16, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
					.SetUndeterminedPressedImage( IMAGE_BRUSH( "Common/Checkbox_Undetermined_Hovered", Icon16x16, FLinearColor( 0.75f, 0.75f, 0.75f ) ) );

				Style->Set( "RichText.Interactive.Details.Checkbox", CheckBoxStyle );
			}
		}

		// Editor example
		{
			// Default text styles
			{
				const FTextBlockStyle RichTextNormal = FTextBlockStyle()
					.SetFont(DEFAULT_FONT("Regular", 11))
					.SetColorAndOpacity(FSlateColor::UseForeground())
					.SetShadowOffset(FVector2D::ZeroVector)
					.SetShadowColorAndOpacity(FLinearColor::Black)
					.SetHighlightColor(FLinearColor(0.02f, 0.3f, 0.0f))
					.SetHighlightShape(BOX_BRUSH("Common/TextBlockHighlightShape", FMargin(3.f /8.f)));
				Style->Set( "RichText.Editor.Text", RichTextNormal );

				{
					const FButtonStyle RichTextHyperlinkButton = FButtonStyle()
						.SetNormal(BORDER_BRUSH("Old/HyperlinkDotted", FMargin(0,0,0,3/16.0f), FSlateColor::UseForeground() ) )
						.SetPressed(FSlateNoResource() )
						.SetHovered(BORDER_BRUSH("Old/HyperlinkUnderline", FMargin(0,0,0,3/16.0f), FSlateColor::UseForeground() ) );

					const FHyperlinkStyle RichTextHyperlink = FHyperlinkStyle()
						.SetUnderlineStyle(RichTextHyperlinkButton)
						.SetTextStyle(RichTextNormal)
						.SetPadding(FMargin(0.0f));
					Style->Set( "RichText.Editor.Hyperlink", RichTextHyperlink );
				}
			}

			// Toolbar
			{
				const FLinearColor NormalColor(FColor(0xffeff3f3));
				const FLinearColor SelectedColor(FColor(0xffdbe4d5));
				const FLinearColor HoverColor(FColor(0xffdbe4e4));
				const FLinearColor DisabledColor(FColor(0xaaaaaa));
				const FLinearColor TextColor(FColor(0xff2c3e50));

				Style->Set( "RichText.Toolbar.HyperlinkImage", new IMAGE_BRUSH( "Testing/hyperlink", Icon16x16, TextColor ) );

				Style->Set( "RichText.Toolbar.TextColor", TextColor );

				Style->Set( "RichText.Toolbar.Text", FTextBlockStyle(NormalText)
					.SetFont( DEFAULT_FONT( "Regular", 10 ) )
					.SetColorAndOpacity( TextColor )
					);

				Style->Set( "RichText.Toolbar.BoldText", FTextBlockStyle(NormalText)
					.SetFont( DEFAULT_FONT( "Bold", 10 ) )
					.SetColorAndOpacity( TextColor )
					);

				Style->Set( "RichText.Toolbar.ItalicText", FTextBlockStyle(NormalText)
					.SetFont( DEFAULT_FONT( "Italic", 10 ) )
					.SetColorAndOpacity( TextColor )
					);

				Style->Set( "RichText.Toolbar.Checkbox", FCheckBoxStyle()
					.SetCheckBoxType(ESlateCheckBoxType::CheckBox)
					.SetUncheckedImage( IMAGE_BRUSH( "Common/CheckBox", Icon16x16, FLinearColor::White ) )
					.SetUncheckedHoveredImage( IMAGE_BRUSH( "Common/CheckBox", Icon16x16, HoverColor ) )
					.SetUncheckedPressedImage( IMAGE_BRUSH( "Common/CheckBox_Hovered", Icon16x16, HoverColor ) )
					.SetCheckedImage( IMAGE_BRUSH( "Common/CheckBox_Checked_Hovered", Icon16x16, FLinearColor::White ) )
					.SetCheckedHoveredImage( IMAGE_BRUSH( "Common/CheckBox_Checked_Hovered", Icon16x16, HoverColor ) )
					.SetCheckedPressedImage( IMAGE_BRUSH( "Common/CheckBox_Checked", Icon16x16, HoverColor ) )
					.SetUndeterminedImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined", Icon16x16, FLinearColor::White ) )
					.SetUndeterminedHoveredImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined_Hovered", Icon16x16, HoverColor ) )
					.SetUndeterminedPressedImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined_Hovered", Icon16x16, FLinearColor::White ) )
					);

				Style->Set( "RichText.Toolbar.ToggleButtonCheckbox", FCheckBoxStyle()
					.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
					.SetUncheckedImage( BOX_BRUSH("Testing/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), NormalColor ) )
					.SetUncheckedHoveredImage( BOX_BRUSH("Testing/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), HoverColor ) )
					.SetUncheckedPressedImage( BOX_BRUSH("Testing/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), HoverColor ) )
					.SetCheckedImage( BOX_BRUSH("Testing/FlatColorSquare",  FVector2D(1.0f, 1.0f), FMargin(1), SelectedColor ) )
					.SetCheckedHoveredImage( BOX_BRUSH("Testing/FlatColorSquare",  FVector2D(1.0f, 1.0f), FMargin(1), HoverColor ) )
					.SetCheckedPressedImage( BOX_BRUSH("Testing/FlatColorSquare",  FVector2D(1.0f, 1.0f), FMargin(1), HoverColor ) )
					);

				const FButtonStyle Button = FButtonStyle()
					.SetNormal( BOX_BRUSH("Testing/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), NormalColor ) )
					.SetHovered( BOX_BRUSH("Testing/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), HoverColor ) )
					.SetPressed( BOX_BRUSH("Testing/FlatColorSquare",  FVector2D(1.0f, 1.0f), FMargin(1), SelectedColor ) )
					.SetNormalPadding( FMargin( 2,2,2,2 ) )
					.SetPressedPadding( FMargin( 2,3,2,1 ) );
				Style->Set( "RichText.Toolbar.Button", Button );

				const FComboButtonStyle ComboButton = FComboButtonStyle()
					.SetButtonStyle(Button)
					.SetDownArrowImage(IMAGE_BRUSH("Common/ComboArrow", Icon8x8))
					.SetMenuBorderBrush(BOX_BRUSH( "Testing/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), NormalColor ))
					.SetMenuBorderPadding(FMargin(0.0f));
				Style->Set( "RichText.Toolbar.ComboButton", ComboButton	);

				{
					const FButtonStyle ComboBoxButton = FButtonStyle()
						.SetNormal( BOX_BRUSH("Testing/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), FLinearColor::White ) )
						.SetHovered( BOX_BRUSH("Testing/FlatColorSquare", FVector2D(1.0f, 1.0f), FMargin(1), FLinearColor::White ) )
						.SetPressed( BOX_BRUSH("Testing/FlatColorSquare",  FVector2D(1.0f, 1.0f), FMargin(1), FLinearColor::White ) )
						.SetNormalPadding( FMargin( 2,2,2,2 ) )
						.SetPressedPadding( FMargin( 2,3,2,1 ) );

					const FComboButtonStyle ComboBoxComboButton = FComboButtonStyle(ComboButton)
						.SetButtonStyle(ComboBoxButton)
						.SetMenuBorderPadding(FMargin(1.0));

					Style->Set( "RichText.Toolbar.ComboBox", FComboBoxStyle()
						.SetComboButtonStyle(ComboBoxComboButton)
						);
				}
			}
		}

		// The War of the Worlds example
		{
			Style->Set( "TheWarOfTheWorlds.Text", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "Regular", 10 ) )
				.SetColorAndOpacity(FLinearColor(FColor( 0xff2c3e50 )))
			);
		}

		// Rainbow example
		{
			Style->Set( "Rainbow.Text", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "Regular", 12 ) )
				.SetColorAndOpacity(FLinearColor(FColor( 0xff2c3e50 )))
			);

			Style->Set( "Rainbow.Text.Red", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "Regular", 12 ) )
				.SetColorAndOpacity(FLinearColor(FColor( 0xffb40000 )))
			);

			Style->Set( "Rainbow.Text.Orange", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "Regular", 12 ) )
				.SetColorAndOpacity(FLinearColor(FColor( 0xffb46100 )))
			);

			Style->Set( "Rainbow.Text.Yellow", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "Regular", 12 ) )
				.SetColorAndOpacity(FLinearColor(FColor( 0xffb2b400 )))
			);

			Style->Set( "Rainbow.Text.Green", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "Regular", 12 ) )
				.SetColorAndOpacity(FLinearColor(FColor( 0xff22b400 )))
			);

			Style->Set( "Rainbow.Text.Blue", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "Regular", 12 ) )
				.SetColorAndOpacity(FLinearColor(FColor( 0xff006ab4 )))
			);
		}
	}

	return Style;
}

#undef RootToContentDir

#endif // #if !UE_BUILD_SHIPPING
