// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"

#if !UE_BUILD_SHIPPING

/**
 * Test slate style
 */
class FTestStyle 
{
public:

	static TSharedRef< class ISlateStyle > Create();

	/** @return the singleton instance. */
	static const ISlateStyle& Get()
	{
		return *( Instance.Get() );
	}

	static void ResetToDefault();

private:

	static void SetStyle( const TSharedRef< class ISlateStyle >& NewStyle );

private:

	/** Singleton instances of this style. */
	static TSharedPtr< class ISlateStyle > Instance;
};

#endif // #if !UE_BUILD_SHIPPING
