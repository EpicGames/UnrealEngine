// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Colors/SColorPicker.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Layout/WidgetPath.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Colors/SColorSlider.h"
#include "Widgets/Colors/SComplexGradient.h"
#include "Widgets/Colors/SSimpleGradient.h"
#include "Widgets/Colors/SEyeDropperButton.h"
#include "Widgets/Colors/SColorWheel.h"
#include "Widgets/Colors/SColorSpectrum.h"
#include "Widgets/Colors/SColorThemes.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Framework/Application/MenuStack.h"

#define LOCTEXT_NAMESPACE "ColorPicker"


/** A default window size for the color picker which looks nice */
const FVector2D SColorPicker::DEFAULT_WINDOW_SIZE = FVector2D(462, 446);

/** The max time allowed for updating before we shut off auto-updating */
const double SColorPicker::MAX_ALLOWED_UPDATE_TIME = 0.1;


/* SColorPicker structors
 *****************************************************************************/

SColorPicker::~SColorPicker()
{
}


/* SColorPicker methods
 *****************************************************************************/

void SColorPicker::Construct( const FArguments& InArgs )
{
	TargetColorAttribute = InArgs._TargetColorAttribute;
	ColorWheelBrush = InArgs._ColorWheelBrush;
	CurrentColorHSV = OldColor = TargetColorAttribute.Get().LinearRGBToHSV();
	CurrentColorRGB = TargetColorAttribute.Get();
	CurrentMode = EColorPickerModes::Wheel;
	bUseAlpha = InArgs._UseAlpha;
	bOnlyRefreshOnMouseUp = InArgs._OnlyRefreshOnMouseUp.Get();
	bOnlyRefreshOnOk = InArgs._OnlyRefreshOnOk.Get();
	OnColorCommitted = InArgs._OnColorCommitted;
	OnColorPickerCancelled = InArgs._OnColorPickerCancelled;
	OnInteractivePickBegin = InArgs._OnInteractivePickBegin;
	OnInteractivePickEnd = InArgs._OnInteractivePickEnd;
	OnColorPickerWindowClosed = InArgs._OnColorPickerWindowClosed;
	ParentWindowPtr = InArgs._ParentWindow.Get();
	DisplayGamma = InArgs._DisplayGamma;
	bClosedViaOkOrCancel = false;
	bValidCreationOverrideExists = InArgs._OverrideColorPickerCreation;
	bClampValue = InArgs._ClampValue;
	OptionalOwningDetailsView = InArgs._OptionalOwningDetailsView.Get().IsValid() ? InArgs._OptionalOwningDetailsView.Get() : nullptr;

	RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &SColorPicker::AnimatePostConstruct ) );

	// We need a parent window to set the close callback
	if (ParentWindowPtr.IsValid())
	{
		ParentWindowPtr.Pin()->SetOnWindowClosed(FOnWindowClosed::CreateSP(this, &SColorPicker::HandleParentWindowClosed));
	}

	bColorPickerIsInlineVersion = InArgs._DisplayInlineVersion;
	bIsInteractive = false;
	bPerfIsTooSlowToUpdate = false;
	bIsThemePanelVisible = true;

	NewColorPreviewImageVisibility = EVisibility::Hidden;
	OldColorPreviewImageVisibility = EVisibility::Hidden;

	BeginAnimation(FLinearColor(ForceInit), CurrentColorHSV);

	if (FPaths::FileExists(GEditorPerProjectIni))
	{
		bool WheelMode = true;
		bool bHexSRGB = true;

		GConfig->GetBool(TEXT("ColorPickerUI"), TEXT("bWheelMode"), WheelMode, GEditorPerProjectIni);
		GConfig->GetBool(TEXT("ColorPickerUI"), TEXT("bSRGBEnabled"), bUseSRGB, GEditorPerProjectIni);
		GConfig->GetBool(TEXT("ColorPickerUI"), TEXT("bHexSRGB"), bHexSRGB, GEditorPerProjectIni);
		GConfig->GetBool(TEXT("ColorPickerUI"), TEXT("bIsThemePanelVisible"), bIsThemePanelVisible, GEditorPerProjectIni);
		
		CurrentMode = WheelMode ? EColorPickerModes::Wheel : EColorPickerModes::Spectrum;
		HexMode = bHexSRGB ? EColorPickerHexMode::SRGB : EColorPickerHexMode::Linear;
	}

	if (InArgs._sRGBOverride.IsSet())
	{
		bUseSRGB = InArgs._sRGBOverride.GetValue();
	}

	if (bColorPickerIsInlineVersion)
	{
		GenerateInlineColorPickerContent();
	}
	else
	{
		GenerateDefaultColorPickerContent(true /*bAdvancedSectionExpanded*/);
	}
}


/* SColorPicker implementation
 *****************************************************************************/


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SColorPicker::GenerateDefaultColorPickerContent( bool bAdvancedSectionExpanded )
{	
	// The height of the gradient bars beneath the sliders
	const FSlateFontInfo SmallFont = FAppStyle::Get().GetFontStyle("ColorPicker.SmallFont");

	this->ChildSlot
	[
		SNew(SVerticalBox)

		// Top Panel, with Color Wheel / Spectrum, Old/New color swatches, and color picker buttons
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				// Overlay displaying either the Color Wheel with Saturation and Value vertical sliders, or just the Color Spectrum widget
				+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SBorder)
							.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
							.Padding(0.0f)
							.OnMouseButtonDown(this, &SColorPicker::HandleColorAreaMouseDown)
							[
								SNew(SOverlay)

								// color wheel
								+ SOverlay::Slot()
									[
										SNew(SHorizontalBox)

										+ SHorizontalBox::Slot()
											.MinWidth(200.0f)
											.MaxWidth(200.0f)
											[
												SAssignNew(ColorWheel, SColorWheel)
													.SelectedColor(this, &SColorPicker::GetCurrentColor)
													.ColorWheelBrush(ColorWheelBrush)
													.Visibility(this, &SColorPicker::HandleColorPickerModeVisibility, EColorPickerModes::Wheel)
													.OnValueChanged(this, &SColorPicker::HandleColorWheelValueChanged)
													.OnMouseCaptureBegin(this, &SColorPicker::HandleInteractiveChangeBegin)
													.OnMouseCaptureEnd(this, &SColorPicker::HandleInteractiveChangeEnd)
													.PreventThrottling(true)
													.ProcessMouseMoveImmediate(bOnlyRefreshOnMouseUp || bOnlyRefreshOnOk)
											]

										+ SHorizontalBox::Slot()
											.AutoWidth()
											.Padding(20.0f, 0.0f, 0.0f, 0.0f)
											[
												// saturation slider
												MakeColorSlider(EColorPickerChannels::Saturation)
											]

										+ SHorizontalBox::Slot()
											.AutoWidth()
											.Padding(20.0f, 0.0f, 0.0f, 0.0f)
											[
												// value slider
												MakeColorSlider(EColorPickerChannels::Value)
											]
									]

								// color spectrum
								+ SOverlay::Slot()
									[
										SNew(SBox)
											.HeightOverride(200.0f)
											.WidthOverride(304.0f)
											[
												SNew(SColorSpectrum)
													.SelectedColor(this, &SColorPicker::GetCurrentColor)
													.Visibility(this, &SColorPicker::HandleColorPickerModeVisibility, EColorPickerModes::Spectrum)
													.OnValueChanged(this, &SColorPicker::HandleColorSpectrumValueChanged)
													.OnMouseCaptureBegin(this, &SColorPicker::HandleInteractiveChangeBegin)
													.OnMouseCaptureEnd(this, &SColorPicker::HandleInteractiveChangeEnd)
													.PreventThrottling(true)
													.ProcessMouseMoveImmediate(bOnlyRefreshOnMouseUp || bOnlyRefreshOnOk)
											]
									]
							]
					]

				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(20.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
							.AutoHeight()
							[
								// color preview
								MakeColorPreviewBox()
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 8.0f, 0.0f, 0.0f)
							[
								// sRGB check box
								SNew(SCheckBox)
									.ToolTipText(LOCTEXT("SRGBCheckboxToolTip", "When enabled, the preview swatch uses sRGB encoding to correct the colors for display.\n"
										"When disabled, the preview shows uncorrected linear colors."))
									.IsChecked(this, &SColorPicker::HandleSRGBCheckBoxIsChecked)
									.OnCheckStateChanged(this, &SColorPicker::HandleSRGBCheckBoxCheckStateChanged)
									[
										SNew(STextBlock)
											.Text(LOCTEXT("SRGBCheckboxLabel", "sRGB Preview"))
									]
							]

						+ SVerticalBox::Slot()
							.MinHeight(28.0f)
							.MaxHeight(28.0f)
							.Padding(0.0f, 8.0f, 0.0f, 0.0f)
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
									.MinWidth(48.0f)
									.MaxWidth(48.0f)
									[
										// mode selector
										SNew(SButton)
											.OnClicked(this, &SColorPicker::HandleColorPickerModeButtonClicked)
											.ContentPadding(FMargin(2.0f, 2.5f))
											.Content()
											[
												SNew(SOverlay)
													.ToolTipText(LOCTEXT("ColorPickerModeEToolTip", "Toggle between color wheel and color spectrum."))

												+ SOverlay::Slot()
													[
														SNew(SImage)
															.Image(FAppStyle::Get().GetBrush("ColorPicker.ModeWheel"))
															.Visibility(this, &SColorPicker::HandleColorPickerModeVisibility, EColorPickerModes::Spectrum)
													]

												+ SOverlay::Slot()
													[
														SNew(SImage)
															.Image(FAppStyle::Get().GetBrush("ColorPicker.ModeSpectrum"))
															.Visibility(this, &SColorPicker::HandleColorPickerModeVisibility, EColorPickerModes::Wheel)
													]
											]
									]

								+ SHorizontalBox::Slot()
									.MinWidth(48.0f)
									.MaxWidth(48.0f)
									.Padding(10.0f, 0.0f, 0.0f, 0.0f)
									[
										// eye dropper
										SNew(SEyeDropperButton)
											.OnValueChanged(this, &SColorPicker::HandleRGBColorChanged)
											.OnBegin(this, &SColorPicker::HandleEyeDropperButtonBegin)
											.OnComplete(this, &SColorPicker::HandleEyeDropperButtonComplete)
											.DisplayGamma(DisplayGamma)
											.Visibility(bValidCreationOverrideExists ? EVisibility::Collapsed : EVisibility::Visible)
									]
							]

						+ SVerticalBox::Slot()
							.MinHeight(28.0f)
							.MaxHeight(28.0f)
							.Padding(0.0f, 8.0f, 0.0f, 0.0f)
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
									.MinWidth(48.0f)
									.MaxWidth(48.0f)
									[
										// Show/Hide Themes Panel
										SNew(SButton)
											.OnClicked(this, &SColorPicker::ToggleThemePanelVisibility)
											.ContentPadding(FMargin(2.0f, 2.5f))
											.ToolTipText(LOCTEXT("ShowHideThemesButtonTooltip", "Toggle visibility of color themes"))
											.Content()
											[
												SNew(SImage)
													.Image(this, &SColorPicker::HandleThemePanelButtonImageBrush)
											]
									]
							]
					]
			]

		// Color Sliders Panel
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 16.0f, 0.0f, 0.0f)
			[
				SNew(SHorizontalBox)

				// RGBA Color Sliders
				+ SHorizontalBox::Slot()
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
							.AutoHeight()
							[
								MakeColorSpinBox(EColorPickerChannels::Red)
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 8.0f, 0.0f, 0.0f)
							[
								MakeColorSpinBox(EColorPickerChannels::Green)
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 8.0f, 0.0f, 0.0f)
							[
								MakeColorSpinBox(EColorPickerChannels::Blue)
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 8.0f, 0.0f, 0.0f)
							[
								MakeColorSpinBox(EColorPickerChannels::Alpha)
							]
					]

				// HSV Color Sliders and & Hexadecimal TextBoxes
				+ SHorizontalBox::Slot()
					.Padding(16.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SVerticalBox)
							
						+ SVerticalBox::Slot()
							.AutoHeight()
							[
								MakeColorSpinBox(EColorPickerChannels::Hue)
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 8.0f, 0.0f, 0.0f)
							[
								MakeColorSpinBox(EColorPickerChannels::Saturation)
							]

						+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0f, 8.0f, 0.0f, 0.0f)
							[
								MakeColorSpinBox(EColorPickerChannels::Value)
							]

						// Hexadecimal Dropdown and TextBox
						+ SVerticalBox::Slot()
							.MinHeight(20)
							.MaxHeight(20)
							.Padding(0.0f, 8.0f, 0.0f, 0.0f)
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
									.HAlign(HAlign_Left)
									.VAlign(VAlign_Center)
									[
										SNew(SComboButton)
											.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("ColorPicker.HexMode"))
											.MenuContent()
											[
												MakeHexModeMenu()
											]
											.ButtonContent()
											[
												SNew(STextBlock)
													.Font(SmallFont)
													.Text(this, &SColorPicker::HandleHexModeButtonText)
											]
									]

								+ SHorizontalBox::Slot()
									.AutoWidth()
									.HAlign(HAlign_Right)
									[
										SNew(SEditableTextBox)
											.MinDesiredWidth(109.0f)
											.Text(this, &SColorPicker::HandleHexBoxText)
											.Font(SmallFont)
											.Padding(FMargin(8.0f, 4.0f, 8.0f, 4.0f))
											.OnTextCommitted(this, &SColorPicker::HandleHexInputTextCommitted)
									]
							]
					]
			]

		// Color Themes Panel
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 16.0f, 0.0f, 0.0f)
			[
 				SNew(SBorder)
 					.BorderImage(FAppStyle::Get().GetBrush("ColorPicker.RecessedBackground"))
 					.Padding(FMargin(8.0f, 8.0f))
					.Visibility(this, &SColorPicker::HandleThemesPanelVisibility)
 					[
						// color theme bar
						SAssignNew(CurrentThemeBar, SThemeColorBlocksBar)
							.ToolTipText(this, &SColorPicker::GetColorThemePanelToolTipText)
							.UseAlpha(bUseAlpha.Get())
							.UseSRGB(SharedThis(this), &SColorPicker::HandleColorPickerUseSRGB)
							.OnSelectColor(this, &SColorPicker::HandleThemeBarColorSelected)
							.OnGetActiveColor(this, &SColorPicker::GetCurrentColor)
					]
			]
			
		// dialog buttons
		+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(0.0f, 16.0f, 0.0f, 0.0f)
			[
				SNew(SUniformGridPanel)
					.MinDesiredSlotHeight(FAppStyle::Get().GetFloat("StandardDialog.MinDesiredSlotHeight"))
					.MinDesiredSlotWidth(FAppStyle::Get().GetFloat("StandardDialog.MinDesiredSlotWidth"))
					.SlotPadding(FAppStyle::Get().GetMargin("StandardDialog.SlotPadding"))
					.Visibility((ParentWindowPtr.IsValid() || bValidCreationOverrideExists) ? EVisibility::Visible : EVisibility::Collapsed)

				+ SUniformGridPanel::Slot(0, 0)
					[
						// ok button
						SNew(SButton)
							.ContentPadding(FAppStyle::Get().GetMargin("StandardDialog.ContentPadding") )
							.HAlign(HAlign_Center)
							.Text(LOCTEXT("OKButton", "OK"))
							.OnClicked(this, &SColorPicker::HandleOkButtonClicked)
					]

				+ SUniformGridPanel::Slot(1, 0)
					[
						// cancel button
						SNew(SButton)
							.ContentPadding(FAppStyle::Get().GetMargin("StandardDialog.ContentPadding") )
							.HAlign(HAlign_Center)
							.Text(LOCTEXT("CancelButton", "Cancel"))
							.OnClicked(this, &SColorPicker::HandleCancelButtonClicked)
					]
			]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

EActiveTimerReturnType SColorPicker::AnimatePostConstruct( double InCurrentTime, float InDeltaTime )
{
	static const float AnimationTime = 0.25f;

	EActiveTimerReturnType TickReturnVal = EActiveTimerReturnType::Continue;
	if ( CurrentTime < AnimationTime )
	{
		CurrentColorHSV = FMath::Lerp( ColorBegin, ColorEnd, CurrentTime / AnimationTime );
		if ( CurrentColorHSV.R < 0.f )
		{
			CurrentColorHSV.R += 360.f;
		}
		else if ( CurrentColorHSV.R > 360.f )
		{
			CurrentColorHSV.R -= 360.f;
		}

		CurrentTime += InDeltaTime;
		if ( CurrentTime >= AnimationTime )
		{
			CurrentColorHSV = ColorEnd;
			TickReturnVal = EActiveTimerReturnType::Stop;
		}

		CurrentColorRGB = CurrentColorHSV.HSVToLinearRGB();
	}

	return TickReturnVal;
}

void SColorPicker::GenerateInlineColorPickerContent()
{
	TSharedRef<SWidget> AlphaSlider = SNullWidget::NullWidget;
	if (bUseAlpha.Get())
	{
		AlphaSlider = MakeColorSlider(EColorPickerChannels::Alpha);
	}

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(ColorWheel, SColorWheel)
					.SelectedColor(this, &SColorPicker::GetCurrentColor)
					.ColorWheelBrush(ColorWheelBrush)
					.Visibility(this, &SColorPicker::HandleColorPickerModeVisibility, EColorPickerModes::Wheel)
					.OnValueChanged(this, &SColorPicker::HandleColorWheelValueChanged)
					.OnMouseCaptureBegin(this, &SColorPicker::HandleInteractiveChangeBegin)
					.OnMouseCaptureEnd(this, &SColorPicker::HandleInteractiveChangeEnd)
					.PreventThrottling(true)
					.ProcessMouseMoveImmediate(bOnlyRefreshOnMouseUp || bOnlyRefreshOnOk)
			]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			[
				// saturation slider
				MakeColorSlider(EColorPickerChannels::Saturation)
			]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			[
				// value slider
				MakeColorSlider(EColorPickerChannels::Value)
			]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			[
				// Alpha slider
				AlphaSlider
			]
	];
}





void SColorPicker::DiscardColor()
{
	if (OnColorPickerCancelled.IsBound())
	{
		// let the user decide what to do about cancel
		OnColorPickerCancelled.Execute( OldColor.HSVToLinearRGB() );
	}
	else
	{	
		SetNewTargetColorHSV(OldColor, true);
	}
}


bool SColorPicker::SetNewTargetColorHSV( const FLinearColor& NewValue, bool bForceUpdate /*= false*/ )
{
	CurrentColorHSV = NewValue;
	CurrentColorRGB = NewValue.HSVToLinearRGB().GetClamped(0.0f, FLT_MAX);

	return ApplyNewTargetColor(bForceUpdate);
}


bool SColorPicker::SetNewTargetColorRGB( const FLinearColor& NewValue, bool bForceUpdate /*= false*/ )
{
	CurrentColorRGB = NewValue.GetClamped(0.0f, FLT_MAX);
	CurrentColorHSV = NewValue.LinearRGBToHSV();

	return ApplyNewTargetColor(bForceUpdate);
}


TSharedRef<SButton> SColorPicker::CreateStandaloneEyeDropperButton(const FEyeDropperButtonArgs& InArgs)
{
	return SNew(SEyeDropperButton)
		.OnValueChanged(InArgs.OnValueChanged)
		.OnBegin(InArgs.OnBegin)
		.OnComplete(InArgs.OnComplete)
		.DisplayGamma(InArgs.DisplayGamma);
}

bool SColorPicker::ApplyNewTargetColor( bool bForceUpdate /*= false*/ )
{
	bool bUpdated = false;

	if ((bForceUpdate || (!bOnlyRefreshOnMouseUp && !bPerfIsTooSlowToUpdate)) && (!bOnlyRefreshOnOk || bColorPickerIsInlineVersion))
	{
		double StartUpdateTime = FPlatformTime::Seconds();
		UpdateColorPickMouseUp();
		double EndUpdateTime = FPlatformTime::Seconds();

		if (EndUpdateTime - StartUpdateTime > MAX_ALLOWED_UPDATE_TIME)
		{
			bPerfIsTooSlowToUpdate = true;
		}

		bUpdated = true;
	}

	return bUpdated;
}


void SColorPicker::UpdateColorPickMouseUp()
{
	if (!bOnlyRefreshOnOk || bColorPickerIsInlineVersion)
	{
		UpdateColorPick();
	}
}


void SColorPicker::UpdateColorPick()
{
	bPerfIsTooSlowToUpdate = false;
	FLinearColor OutColor = CurrentColorRGB;

	OnColorCommitted.ExecuteIfBound(OutColor);
	
	// This callback is only necessary for wx backwards compatibility
	FCoreDelegates::ColorPickerChanged.Broadcast();
}


void SColorPicker::BeginAnimation( FLinearColor Start, FLinearColor End )
{
	ColorEnd = End;
	ColorBegin = Start;
	CurrentTime = 0.f;
	
	// wraparound with hue
	float HueDif = FMath::Abs(ColorBegin.R - ColorEnd.R);
	if (FMath::Abs(ColorBegin.R + 360.f - ColorEnd.R) < HueDif)
	{
		ColorBegin.R += 360.f;
	}
	else if (FMath::Abs(ColorBegin.R - 360.f - ColorEnd.R) < HueDif)
	{
		ColorBegin.R -= 360.f;
	}
}


void SColorPicker::HideSmallTrash()
{
	// Deprecated function
}


void SColorPicker::ShowSmallTrash()
{
	// Deprecated function
}


/* SColorPicker implementation
 *****************************************************************************/

void SColorPicker::CycleMode()
{
	if (CurrentMode == EColorPickerModes::Spectrum)
	{
		CurrentMode = EColorPickerModes::Wheel;
	}
	else
	{
		CurrentMode = EColorPickerModes::Spectrum;
	}
}


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SColorPicker::MakeColorSlider( EColorPickerChannels Channel ) const
{
	FText SliderTooltip;

	switch (Channel)
	{
	case EColorPickerChannels::Red:			SliderTooltip = LOCTEXT("RedSliderToolTip", "Red"); break;
	case EColorPickerChannels::Green:		SliderTooltip = LOCTEXT("GreenSliderToolTip", "Green"); break;
	case EColorPickerChannels::Blue:		SliderTooltip = LOCTEXT("BlueSliderToolTip", "Blue"); break;
	case EColorPickerChannels::Alpha:		SliderTooltip = LOCTEXT("AlphaSliderToolTip", "Alpha"); break;
	case EColorPickerChannels::Hue:			SliderTooltip = LOCTEXT("HueSliderToolTip", "Hue"); break;
	case EColorPickerChannels::Saturation:	SliderTooltip = LOCTEXT("SaturationSliderToolTip", "Saturation"); break;
	case EColorPickerChannels::Value:		SliderTooltip = LOCTEXT("ValueSliderToolTip", "Value"); break;
	default:
		return SNullWidget::NullWidget;
	}

	return SNew(SColorSlider)
		.Orientation(Orient_Vertical)
		.ToolTipText(SliderTooltip)
		.MinSliderValue(0.0f)
		.MaxSliderValue(Channel == EColorPickerChannels::Hue ? 359.999f : 1.0f)
		.Delta(Channel == EColorPickerChannels::Hue ? 1.0f : 0.001f)
		.SupportDynamicSliderMaxValue(Channel == EColorPickerChannels::Hue ? false : true)
		.HasAlphaBackground(Channel == EColorPickerChannels::Alpha)
		.UseSRGB(this, &SColorPicker::HandleColorPickerUseSRGB)
		.GradientColors(this, &SColorPicker::GetGradientColors, Channel)
		.Value(this, &SColorPicker::HandleColorSpinBoxValue, Channel)
		.Visibility(this, &SColorPicker::HandleColorPickerModeVisibility, EColorPickerModes::Wheel)
		.OnBeginSliderMovement(const_cast<SColorPicker*>(this), &SColorPicker::HandleInteractiveChangeBegin)
		.OnEndSliderMovement(const_cast<SColorPicker*>(this), &SColorPicker::HandleInteractiveChangeEnd)
		.OnValueChanged(const_cast<SColorPicker*>(this), &SColorPicker::HandleColorSpinBoxValueChanged, Channel)
		.PreventThrottling(true);
}


TSharedRef<SWidget> SColorPicker::MakeColorSpinBox( EColorPickerChannels Channel ) const
{
	if ((Channel == EColorPickerChannels::Alpha) && !bUseAlpha.Get())
	{
		return SNullWidget::NullWidget;
	}

	const float HDRMaxValue = bClampValue ? 1.f : FLT_MAX;

	float MaxValue;
	FText SliderLabel;
	FText SliderTooltip;

	switch (Channel)
	{
	case EColorPickerChannels::Red:
		MaxValue = HDRMaxValue;
		SliderLabel = LOCTEXT("RedSliderLabel", "R");
		SliderTooltip = LOCTEXT("RedSliderToolTip", "Red");
		break;
		
	case EColorPickerChannels::Green:
		MaxValue = HDRMaxValue;
		SliderLabel = LOCTEXT("GreenSliderLabel", "G");
		SliderTooltip = LOCTEXT("GreenSliderToolTip", "Green");
		break;
		
	case EColorPickerChannels::Blue:
		MaxValue = HDRMaxValue;
		SliderLabel = LOCTEXT("BlueSliderLabel", "B");
		SliderTooltip = LOCTEXT("BlueSliderToolTip", "Blue");
		break;
		
	case EColorPickerChannels::Alpha:
		MaxValue = HDRMaxValue;
		SliderLabel = LOCTEXT("AlphaSliderLabel", "A");
		SliderTooltip = LOCTEXT("AlphaSliderToolTip", "Alpha");
		break;
		
	case EColorPickerChannels::Hue:
		MaxValue = HDRMaxValue;
		SliderLabel = LOCTEXT("HueSliderLabel", "H");
		SliderTooltip = LOCTEXT("HueSliderToolTip", "Hue");
		break;
		
	case EColorPickerChannels::Saturation:
		MaxValue = HDRMaxValue;
		SliderLabel = LOCTEXT("SaturationSliderLabel", "S");
		SliderTooltip = LOCTEXT("SaturationSliderToolTip", "Saturation");
		break;
		
	case EColorPickerChannels::Value:
		MaxValue = HDRMaxValue;
		SliderLabel = LOCTEXT("ValueSliderLabel", "V");
		SliderTooltip = LOCTEXT("ValueSliderToolTip", "Value");
		break;

	default:
		return SNullWidget::NullWidget;
	}

	return SNew(SColorSlider)
		.Label(SliderLabel)
		.ToolTipText(SliderTooltip)
		.MinSpinBoxValue(0.0f)
		.MaxSpinBoxValue(MaxValue)
		.MinSliderValue(0.0f)
		.MaxSliderValue(Channel == EColorPickerChannels::Hue ? 359.999f : 1.0f)
		.Delta(Channel == EColorPickerChannels::Hue ? 1.0f : 0.001f)
		.SupportDynamicSliderMaxValue(Channel == EColorPickerChannels::Hue ? false : true)
		.HasAlphaBackground(Channel == EColorPickerChannels::Alpha)
		.UseSRGB(this, &SColorPicker::HandleColorPickerUseSRGB)
		.GradientColors(this, &SColorPicker::GetGradientColors, Channel)
		.Value(this, &SColorPicker::HandleColorSpinBoxValue, Channel)
		.OnBeginSliderMovement(const_cast<SColorPicker*>(this), &SColorPicker::HandleInteractiveChangeBegin)
		.OnEndSliderMovement(const_cast<SColorPicker*>(this), &SColorPicker::HandleInteractiveChangeEnd)
		.OnBeginSpinBoxMovement(const_cast<SColorPicker*>(this), &SColorPicker::HandleInteractiveChangeBegin)
		.OnEndSpinBoxMovement(const_cast<SColorPicker*>(this), &SColorPicker::HandleInteractiveChangeEnd)
		.OnValueChanged(const_cast<SColorPicker*>(this), &SColorPicker::HandleColorSpinBoxValueChanged, Channel);
}


TSharedRef<SWidget> SColorPicker::MakeColorPreviewBox() const
{
	return SNew(SVerticalBox)

	+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
				[
					// new color (alpha)
					SNew(SColorBlock)
						.ColorIsHSV(true)
						.ShowBackgroundForAlpha(true)
						.AlphaDisplayMode(SharedThis(this), &SColorPicker::HandleColorPreviewAlphaMode)
						.Color(this, &SColorPicker::GetCurrentColor)
						.ToolTipText(LOCTEXT("NewColorBlockToolTip", "Preview of the currently selected color"))
						.UseSRGB(SharedThis(this), &SColorPicker::HandleColorPickerUseSRGB)
						.Size(FVector2D(106.0, 32.0))
						.CornerRadius(FVector4(4.0, 4.0, 4.0, 4.0))
				]

			+ SOverlay::Slot()
				[
					SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "ColorPicker.ColorPreviewButton")
						.OnClicked(const_cast<SColorPicker*>(this), &SColorPicker::HandleNewColorPreviewClicked)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.ToolTipText(LOCTEXT("NewColorButtonToolTip", "Add the currently selected color to the current color theme"))
						.Visibility(this, &SColorPicker::HandleColorPreviewButtonVisibility)
						.OnHovered(const_cast<SColorPicker*>(this), &SColorPicker::SetNewColorPreviewImageVisibility, EVisibility::Visible)
						.OnUnhovered(const_cast<SColorPicker*>(this), &SColorPicker::SetNewColorPreviewImageVisibility, EVisibility::Hidden)
						.Content()
						[
							SNew(SImage)
								.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
								.Visibility(const_cast<SColorPicker*>(this), &SColorPicker::GetNewColorPreviewImageVisibility)
						]
				]
		]

	+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 2.0f, 0.0f, 0.0f)
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
				[
					// Old color
					SNew(SColorBlock)
						.ColorIsHSV(true)
						.ShowBackgroundForAlpha(true)
						.AlphaDisplayMode(SharedThis(this), &SColorPicker::HandleColorPreviewAlphaMode)
						.Color(OldColor)
						.ToolTipText(LOCTEXT("OldColorBlockToolTip", "Preview of the previously selected color"))
						.UseSRGB(SharedThis(this), &SColorPicker::HandleColorPickerUseSRGB)
						.Size(FVector2D(106.0, 32.0))
						.CornerRadius(FVector4(4.0, 4.0, 4.0, 4.0))
				]

			+ SOverlay::Slot()
				[
					SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "ColorPicker.ColorPreviewButton")
						.OnClicked(const_cast<SColorPicker*>(this), &SColorPicker::HandleOldColorPreviewClicked)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.ToolTipText(LOCTEXT("OldColorButtonToolTip", "Add the previously selected color to the current color theme"))
						.Visibility(this, &SColorPicker::HandleColorPreviewButtonVisibility)
						.OnHovered(const_cast<SColorPicker*>(this), &SColorPicker::SetOldColorPreviewImageVisibility, EVisibility::Visible)
						.OnUnhovered(const_cast<SColorPicker*>(this), &SColorPicker::SetOldColorPreviewImageVisibility, EVisibility::Hidden)
						.Content()
						[
							SNew(SImage)
								.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
								.Visibility(const_cast<SColorPicker*>(this), &SColorPicker::GetOldColorPreviewImageVisibility)
						]
				]
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


/* SColorPicker callbacks
 *****************************************************************************/

FLinearColor SColorPicker::GetGradientEndColor( EColorPickerChannels Channel ) const
{
	switch (Channel)
	{
	case EColorPickerChannels::Red:			return FLinearColor::Red;
	case EColorPickerChannels::Green:		return FLinearColor::Green;
	case EColorPickerChannels::Blue:		return FLinearColor::Blue;
	case EColorPickerChannels::Alpha:		return FLinearColor(CurrentColorHSV.R, CurrentColorHSV.G, CurrentColorHSV.B, 1.0f).HSVToLinearRGB();
	case EColorPickerChannels::Saturation:	return FLinearColor(CurrentColorHSV.R, 1.0f, 1.0f, 1.0f).HSVToLinearRGB();
	case EColorPickerChannels::Value:		return FLinearColor(CurrentColorHSV.R, CurrentColorHSV.G, 1.0f, 1.0f).HSVToLinearRGB();
	default:								return FLinearColor();
	}
}


FLinearColor SColorPicker::GetGradientStartColor( EColorPickerChannels Channel ) const
{
	switch (Channel)
	{
	case EColorPickerChannels::Red:			return FLinearColor::Black;
	case EColorPickerChannels::Green:		return FLinearColor::Black;
	case EColorPickerChannels::Blue:		return FLinearColor::Black;
	case EColorPickerChannels::Alpha:		return FLinearColor::Transparent;
	case EColorPickerChannels::Saturation:	return FLinearColor(CurrentColorHSV.R, 0.0f, 1.0f, 1.0f).HSVToLinearRGB();
	case EColorPickerChannels::Value:		return FLinearColor(CurrentColorHSV.R, CurrentColorHSV.G, 0.0f, 1.0f).HSVToLinearRGB();
	default:								return FLinearColor();
	}
}

TArray<FLinearColor> SColorPicker::GetGradientColors( EColorPickerChannels Channel ) const
{
	TArray<FLinearColor> Colors;
	if (Channel == EColorPickerChannels::Hue)
	{
		for (int32 i = 0; i < 10; ++i)
		{
			Colors.Add(FLinearColor((i % 9) * 40.f, 1.f, 1.f).HSVToLinearRGB());
		}
	}
	else
	{
		const FLinearColor StartColor = GetGradientStartColor(Channel);
		const FLinearColor EndColor = GetGradientEndColor(Channel);

		constexpr int32 NumSteps = 10;
		constexpr float StepSize = 1.0f / NumSteps;
		for (int32 Step = 0; Step <= NumSteps; ++Step)
		{
			const float Alpha = Step * StepSize;
			Colors.Add(FMath::Lerp(StartColor, EndColor, Alpha));
		}
	}
	return Colors;
}


EColorBlockAlphaDisplayMode SColorPicker::HandleColorPreviewAlphaMode() const
{
	return bUseAlpha.Get() ? EColorBlockAlphaDisplayMode::SeparateReverse : EColorBlockAlphaDisplayMode::Ignore;
}


FReply SColorPicker::HandleCancelButtonClicked()
{
	bClosedViaOkOrCancel = true;

	DiscardColor();
	if (SColorPicker::OnColorPickerDestroyOverride.IsBound())
	{
		SColorPicker::OnColorPickerDestroyOverride.Execute();
	}
	else
	{
		ParentWindowPtr.Pin()->RequestDestroyWindow();
	}

	return FReply::Handled();
}


EVisibility SColorPicker::HandleColorPickerModeVisibility( EColorPickerModes Mode ) const
{
	return (CurrentMode == Mode) ? EVisibility::Visible : EVisibility::Hidden;
}


EVisibility SColorPicker::HandleThemesPanelVisibility() const
{
	return bIsThemePanelVisible ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SColorPicker::GetColorThemePanelToolTipText() const
{
	return (CurrentThemeBar->IsRecentsThemeActive()) ? LOCTEXT("RecentsThemeToolTipText", "Recently used colors") : LOCTEXT("ColorThemeToolTipText", "Current Color Theme");
}

const FSlateBrush* SColorPicker::HandleThemePanelButtonImageBrush() const
{
	return bIsThemePanelVisible ? FAppStyle::Get().GetBrush("ColorPicker.ColorThemes") : FAppStyle::Get().GetBrush("ColorPicker.ColorThemesOff");
}

void SColorPicker::HandleThemeBarColorSelected(FLinearColor NewValue)
{
	// Force the alpha component to 1 when we don't care about the alpha
	if (!bUseAlpha.Get())
	{
		NewValue.A = 1.0f;
	}

	BeginAnimation(CurrentColorHSV, NewValue);
	SetNewTargetColorHSV(NewValue, true);
}


FLinearColor SColorPicker::HandleColorSliderEndColor( EColorPickerChannels Channel ) const
{
	switch (Channel)
	{
	case EColorPickerChannels::Red:			return FLinearColor(0.0f, 0.0f, 0.0f, 1.0f);
	case EColorPickerChannels::Green:		return FLinearColor(0.0f, 0.0f, 0.0f, 1.0f);
	case EColorPickerChannels::Blue:		return FLinearColor(0.0f, 0.0f, 0.0f, 1.0f);
	case EColorPickerChannels::Alpha:		return FLinearColor(CurrentColorRGB.R, CurrentColorRGB.G, CurrentColorRGB.B, 0.0f);
	case EColorPickerChannels::Saturation:	return FLinearColor(CurrentColorHSV.R, 0.0f, 1.0f, 1.0f).HSVToLinearRGB();
	case EColorPickerChannels::Value:		return FLinearColor(CurrentColorHSV.R, CurrentColorHSV.G, 0.0f, 1.0f).HSVToLinearRGB();
	default:								return FLinearColor();
	}
}


FLinearColor SColorPicker::HandleColorSliderStartColor( EColorPickerChannels Channel ) const
{
	switch (Channel)
	{
	case EColorPickerChannels::Red:			return FLinearColor(1.0f, 0.0f, 0.0f, 1.0f);
	case EColorPickerChannels::Green:		return FLinearColor(0.0f, 1.0f, 0.0f, 1.0f);
	case EColorPickerChannels::Blue:		return FLinearColor(0.0f, 0.0f, 1.0f, 1.0f);
	case EColorPickerChannels::Alpha:		return FLinearColor(CurrentColorRGB.R, CurrentColorRGB.G, CurrentColorRGB.B, 1.0f);
	case EColorPickerChannels::Saturation:	return FLinearColor(CurrentColorHSV.R, 1.0f, 1.0f, 1.0f).HSVToLinearRGB();
	case EColorPickerChannels::Value:		return FLinearColor(CurrentColorHSV.R, CurrentColorHSV.G, 1.0f, 1.0f).HSVToLinearRGB();
	default:								return FLinearColor();
	}
}


void SColorPicker::HandleColorWheelValueChanged(FLinearColor NewValue)
{
	// In this color, R = H, G = S, B = V
	if (FMath::IsNearlyZero(NewValue.B))
	{
		NewValue.B = 1.0f;
	}
	if (!bUseAlpha.Get() || FMath::IsNearlyZero(NewValue.A))
	{
		NewValue.A = 1.0f;
	}
	SetNewTargetColorHSV(NewValue);
}


void SColorPicker::HandleColorSpectrumValueChanged(FLinearColor NewValue)
{
	SetNewTargetColorHSV(NewValue);
}


float SColorPicker::HandleColorSpinBoxValue( EColorPickerChannels Channel ) const
{
	switch (Channel)
	{
	case EColorPickerChannels::Red:			return CurrentColorRGB.R;
	case EColorPickerChannels::Green:		return CurrentColorRGB.G;
	case EColorPickerChannels::Blue:		return CurrentColorRGB.B;
	case EColorPickerChannels::Alpha:		return CurrentColorRGB.A;
	case EColorPickerChannels::Hue:			return CurrentColorHSV.R;
	case EColorPickerChannels::Saturation:	return CurrentColorHSV.G;
	case EColorPickerChannels::Value:		return CurrentColorHSV.B;
	default:								return 0.0f;
	}
}


void SColorPicker::HandleColorSpinBoxValueChanged( float NewValue, EColorPickerChannels Channel )
{
	int32 ComponentIndex;
	bool IsHSV = false;

	switch (Channel)
	{
	case EColorPickerChannels::Red:			ComponentIndex = 0; break;
	case EColorPickerChannels::Green:		ComponentIndex = 1; break;
	case EColorPickerChannels::Blue:		ComponentIndex = 2; break;
	case EColorPickerChannels::Alpha:		ComponentIndex = 3; break;
	case EColorPickerChannels::Hue:			ComponentIndex = 0; IsHSV = true; NewValue = FMath::Modulo(NewValue, 360.0f); break;
	case EColorPickerChannels::Saturation:	ComponentIndex = 1; IsHSV = true; break;
	case EColorPickerChannels::Value:		ComponentIndex = 2; IsHSV = true; break;
	default:								
		return;
	}

	FLinearColor& NewColor = IsHSV ? CurrentColorHSV : CurrentColorRGB;

	if (FMath::IsNearlyEqual(NewValue, NewColor.Component(ComponentIndex), KINDA_SMALL_NUMBER))
	{
		return;
	}

	NewColor.Component(ComponentIndex) = NewValue;

	if (IsHSV)
	{
		SetNewTargetColorHSV(NewColor, !bIsInteractive);
	}
	else
	{
		SetNewTargetColorRGB(NewColor, !bIsInteractive);
	}
}

void SColorPicker::HandleEyeDropperButtonBegin()
{
	if (ColorWheel)
	{
		ColorWheel->ShowSelector(false);
	}
	
	HandleInteractiveChangeBegin();
}

void SColorPicker::HandleEyeDropperButtonComplete(bool bCancelled)
{
	if (ColorWheel)
	{
		ColorWheel->ShowSelector(true);
	}
	
	bIsInteractive = false;

	if (bCancelled)
	{
		SetNewTargetColorHSV(OldColor, true);
	}

	if (bOnlyRefreshOnMouseUp || bPerfIsTooSlowToUpdate)
	{
		UpdateColorPick();
	}

	OnInteractivePickEnd.ExecuteIfBound();
}


FText SColorPicker::HandleHexBoxText() const
{
	const bool bSRGB = (HexMode == EColorPickerHexMode::SRGB);
	return FText::FromString(CurrentColorRGB.ToFColor(bSRGB).ToHex());
}


void SColorPicker::HandleHexInputTextCommitted(const FText& Text, ETextCommit::Type CommitType)
{
	if (!Text.IsEmpty() && ((CommitType == ETextCommit::OnEnter) || (CommitType == ETextCommit::OnUserMovedFocus)))
	{
		FColor Color = FColor::FromHex(Text.ToString());
		float Red = Color.R / 255.0f;
		float Green = Color.G / 255.0f;
		float Blue = Color.B / 255.0f;
		float Alpha = Color.A / 255.0f;

		if (HexMode == EColorPickerHexMode::SRGB)
		{
			Red = Red <= 0.04045f ? Red / 12.92f : FMath::Pow((Red + 0.055f) / 1.055f, 2.4f);
			Green = Green <= 0.04045f ? Green / 12.92f : FMath::Pow((Green + 0.055f) / 1.055f, 2.4f);
			Blue = Blue <= 0.04045f ? Blue / 12.92f : FMath::Pow((Blue + 0.055f) / 1.055f, 2.4f);
		}

		SetNewTargetColorRGB(FLinearColor(Red, Green, Blue, Alpha), false);
	}
}


TSharedRef<SWidget> SColorPicker::MakeHexModeMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("HexMenuText_SRGB", "Hex sRGB"),
		LOCTEXT("HexMenuToolTip_SRGB", "Represents the color being created using sRGB encoding.\n"
			"This format matches the hex color values typically used in web development and image editing software."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SColorPicker::OnHexModeSelected, EColorPickerHexMode::SRGB),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return HexMode == EColorPickerHexMode::SRGB; })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("HexMenuText_Linear", "Hex Linear"),
		LOCTEXT("HexMenuToolTip_Linear", "Represents the color being created using linear color values.\n"
			"Note that linear hex values have less precision for darker colors."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SColorPicker::OnHexModeSelected, EColorPickerHexMode::Linear),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return HexMode == EColorPickerHexMode::Linear; })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	return MenuBuilder.MakeWidget();
}

FText SColorPicker::HandleHexModeButtonText() const
{
	if (HexMode == EColorPickerHexMode::SRGB)
	{
		return LOCTEXT("HexMenuText_SRGB", "Hex sRGB");
	}
	else if (HexMode == EColorPickerHexMode::Linear)
	{
		return LOCTEXT("HexMenuText_Linear", "Hex Linear");
	}

	return FText::GetEmpty();
}

void SColorPicker::OnHexModeSelected(EColorPickerHexMode InHexMode)
{
	HexMode = InHexMode;

	if (FPaths::FileExists(GEditorPerProjectIni))
	{
		GConfig->SetBool(TEXT("ColorPickerUI"), TEXT("bHexSRGB"), HexMode == EColorPickerHexMode::SRGB, GEditorPerProjectIni);
	}
}


void SColorPicker::HandleHSVColorChanged( FLinearColor NewValue )
{
	SetNewTargetColorHSV(NewValue);
}


void SColorPicker::HandleInteractiveChangeBegin()
{
	if( bIsInteractive && OnInteractivePickEnd.IsBound() )
	{
		OnInteractivePickEnd.Execute();
	}

	OnInteractivePickBegin.ExecuteIfBound();
	bIsInteractive = true;
}


void SColorPicker::HandleInteractiveChangeEnd()
{
	HandleInteractiveChangeEnd(0.0f);
}


void SColorPicker::HandleInteractiveChangeEnd( float NewValue )
{
	bIsInteractive = false;

	UpdateColorPickMouseUp();
	OnInteractivePickEnd.ExecuteIfBound();
}


FReply SColorPicker::HandleColorAreaMouseDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		CycleMode();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}


FReply SColorPicker::HandleColorPickerModeButtonClicked()
{
	CycleMode();

	if (FPaths::FileExists(GEditorPerProjectIni))
	{
		GConfig->SetBool(TEXT("ColorPickerUI"), TEXT("bWheelMode"), CurrentMode == EColorPickerModes::Wheel, GEditorPerProjectIni);
	}

	return FReply::Handled();
}


EVisibility SColorPicker::HandleColorPreviewButtonVisibility() const
{
	return CurrentThemeBar->IsRecentsThemeActive() ? EVisibility::Hidden : EVisibility::Visible;
}


void SColorPicker::SetNewColorPreviewImageVisibility(EVisibility InButtonVisibility)
{
	NewColorPreviewImageVisibility = InButtonVisibility;
}


void SColorPicker::SetOldColorPreviewImageVisibility(EVisibility InButtonVisibility)
{
	OldColorPreviewImageVisibility = InButtonVisibility;
}


EVisibility SColorPicker::GetNewColorPreviewImageVisibility() const
{
	return NewColorPreviewImageVisibility;
}


EVisibility SColorPicker::GetOldColorPreviewImageVisibility() const
{
	return OldColorPreviewImageVisibility;
}


FReply SColorPicker::HandleNewColorPreviewClicked()
{
	CurrentThemeBar->AddNewColorBlock(CurrentColorHSV, 0);
	return FReply::Handled();
}


FReply SColorPicker::HandleOldColorPreviewClicked()
{
	CurrentThemeBar->AddNewColorBlock(OldColor, 0);
	return FReply::Handled();
}


FReply SColorPicker::ToggleThemePanelVisibility()
{
	bIsThemePanelVisible = !bIsThemePanelVisible;

	if (FPaths::FileExists(GEditorPerProjectIni))
	{
		GConfig->SetBool(TEXT("ColorPickerUI"), TEXT("bIsThemePanelVisible"), bIsThemePanelVisible, GEditorPerProjectIni);
	}

	return FReply::Handled();
}


FReply SColorPicker::HandleOkButtonClicked()
{
	bClosedViaOkOrCancel = true;

	UpdateColorPick();

	if (OldColor != CurrentColorHSV)
	{
		CurrentThemeBar->AddToRecents(CurrentColorHSV);
	}

	if (SColorPicker::OnColorPickerDestroyOverride.IsBound())
	{
		SColorPicker::OnColorPickerDestroyOverride.Execute();
	}
	else
	{
		ParentWindowPtr.Pin()->RequestDestroyWindow();
	}
	return FReply::Handled();
}


bool SColorPicker::HandleColorPickerUseSRGB() const
{
	return bUseSRGB;
}


void SColorPicker::HandleParentWindowClosed( const TSharedRef<SWindow>& Window )
{
	check(Window == ParentWindowPtr.Pin());

	// End picking interaction if still active
	if( bIsInteractive && OnInteractivePickEnd.IsBound() )
	{
		OnInteractivePickEnd.Execute();
		bIsInteractive = false;
	}

	// If we weren't closed via the OK or Cancel button, we need to perform the default close action
	if (!bClosedViaOkOrCancel && bOnlyRefreshOnOk)
	{
		DiscardColor();
	}

	// We always have to call the close callback
	if (OnColorPickerWindowClosed.IsBound())
	{
		OnColorPickerWindowClosed.Execute(Window);
	}
}


void SColorPicker::HandleRGBColorChanged( FLinearColor NewValue )
{
	SetNewTargetColorRGB(NewValue);
}


void SColorPicker::HandleSRGBCheckBoxCheckStateChanged( ECheckBoxState InIsChecked )
{
	bUseSRGB = (InIsChecked == ECheckBoxState::Checked);

	if (FPaths::FileExists(GEditorPerProjectIni))
	{
		GConfig->SetBool(TEXT("ColorPickerUI"), TEXT("bSRGBEnabled"), bUseSRGB, GEditorPerProjectIni);
	}
}


ECheckBoxState SColorPicker::HandleSRGBCheckBoxIsChecked() const
{
	return bUseSRGB ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}


// Static delegates to access whether or not the override is bound in the global Open/Destroy functions
SColorPicker::FOnColorPickerCreationOverride SColorPicker::OnColorPickerNonModalCreateOverride;
SColorPicker::FOnColorPickerDestructionOverride SColorPicker::OnColorPickerDestroyOverride;

/* Global functions
 *****************************************************************************/

/** A static color picker that everything should use. */
static TWeakPtr<SWindow> ColorPickerWindow;

static TWeakPtr<SColorPicker> GlobalColorPicker;

TSharedPtr<SColorPicker> GetColorPicker()
{
	if (GlobalColorPicker.IsValid())
	{
		return GlobalColorPicker.Pin();
	}
	else
	{
		return nullptr;
	}
}

bool OpenColorPicker(const FColorPickerArgs& Args)
{
	DestroyColorPicker();
	bool Result = false;

	// Consoles do not support opening new windows
#if PLATFORM_DESKTOP
	FLinearColor OldColor = Args.InitialColor;
	ensureMsgf(Args.OnColorCommitted.IsBound(), TEXT("OnColorCommitted should be bound to set the color."));
		
	// Determine the position of the window so that it will spawn near the mouse, but not go off the screen.
	const FVector2D CursorPos = FSlateApplication::Get().GetCursorPos();
	const FSlateRect Anchor(CursorPos.X, CursorPos.Y, CursorPos.X, CursorPos.Y);

	// Because the window has not yet been created, its desired size is still unknown. 
	// This estimate is the size of the window with 4 rows of color theme blocks, which should be large enough in most cases to compute a reasonable summon location.
	const FVector2D PaddingForColorTheme = FVector2D(0, 130);
	const FVector2D WindowSizeEstimate = SColorPicker::DEFAULT_WINDOW_SIZE + PaddingForColorTheme;
	const FVector2D AdjustedSummonLocation = FSlateApplication::Get().CalculatePopupWindowPosition( Anchor, WindowSizeEstimate, true, FVector2D::ZeroVector, Orient_Horizontal );

	// Only override the color picker window creation behavior if we are not creating a modal color picker
	const bool bOverrideNonModalCreation = (SColorPicker::OnColorPickerNonModalCreateOverride.IsBound() && !Args.bIsModal);

	TSharedPtr<SWindow> Window = nullptr;
	TSharedRef<SBorder> WindowContent = SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			.Padding(FMargin(16.0f, 16.0f));
	
	bool bNeedToAddWindow = true;
	if (!bOverrideNonModalCreation)
	{
		if (Args.bOpenAsMenu && !Args.bIsModal && Args.ParentWidget.IsValid())
		{
			Window = FSlateApplication::Get().PushMenu(
				Args.ParentWidget.ToSharedRef(),
				FWidgetPath(),
				WindowContent,
				AdjustedSummonLocation,
				FPopupTransitionEffect(FPopupTransitionEffect::None),
				false,
				FVector2D(0.f,0.f),
				EPopupMethod::CreateNewWindow,
				false)->GetOwnedWindow();

			bNeedToAddWindow = false;
		}
		else
		{
			Window = SNew(SWindow)
				.AutoCenter(EAutoCenter::None)
				.ScreenPosition(AdjustedSummonLocation)
				.SupportsMaximize(false)
				.SupportsMinimize(false)
				.SizingRule(ESizingRule::Autosized)
				.Title(LOCTEXT("WindowHeader", "Color Picker"))
				[
					WindowContent
				];
		}
	}

	TSharedRef<SColorPicker> CreatedColorPicker = SNew(SColorPicker)
		.TargetColorAttribute(OldColor)
		.UseAlpha(Args.bUseAlpha)
		.OnlyRefreshOnMouseUp(Args.bOnlyRefreshOnMouseUp && !Args.bIsModal)
		.OnlyRefreshOnOk(Args.bOnlyRefreshOnOk || Args.bIsModal)
		.OnColorCommitted(Args.OnColorCommitted)
		.OnColorPickerCancelled(Args.OnColorPickerCancelled)
		.OnInteractivePickBegin(Args.OnInteractivePickBegin)
		.OnInteractivePickEnd(Args.OnInteractivePickEnd)
		.OnColorPickerWindowClosed(Args.OnColorPickerWindowClosed)
		.ParentWindow(Window)
		.DisplayGamma(Args.DisplayGamma)
		.sRGBOverride(Args.sRGBOverride)
		.OverrideColorPickerCreation(bOverrideNonModalCreation)
		.OptionalOwningDetailsView(Args.OptionalOwningDetailsView);
	
	// If the color picker requested is modal, don't override the behavior even if the delegate is bound
	if (bOverrideNonModalCreation)
	{
		SColorPicker::OnColorPickerNonModalCreateOverride.Execute(CreatedColorPicker);

		Result = true;

		//hold on to the window created for external use...
		ColorPickerWindow = Window;
	}
	else
	{
		WindowContent->SetContent(CreatedColorPicker);

		if (Args.bIsModal)
		{
			FSlateApplication::Get().AddModalWindow(Window.ToSharedRef(), Args.ParentWidget);
			Result = true;
		}
		else if (bNeedToAddWindow)
		{
			if (Args.ParentWidget.IsValid())
			{
				// Find the window of the parent widget
				FWidgetPath WidgetPath;
				if (FSlateApplication::Get().GeneratePathToWidgetUnchecked(Args.ParentWidget.ToSharedRef(), WidgetPath))
				{
					Window = FSlateApplication::Get().AddWindowAsNativeChild(Window.ToSharedRef(), WidgetPath.GetWindow());
					Result = true;
				}
			}
			else
			{
				Window = FSlateApplication::Get().AddWindow(Window.ToSharedRef());
				Result = true;
			}
		}

		//hold on to the window created for external use...
		ColorPickerWindow = Window;
	}
	GlobalColorPicker = CreatedColorPicker;
#endif

	return Result;
}


/**
 * Destroys the current color picker. Necessary if the values the color picker
 * currently targets become invalid.
 */
void DestroyColorPicker()
{
	if (ColorPickerWindow.IsValid())
	{
		if (SColorPicker::OnColorPickerDestroyOverride.IsBound())
		{
			SColorPicker::OnColorPickerDestroyOverride.Execute();
		}
		else
		{
			ColorPickerWindow.Pin()->RequestDestroyWindow();
		}
		ColorPickerWindow.Reset();
		GlobalColorPicker.Reset();
	}
}


#undef LOCTEXT_NAMESPACE
