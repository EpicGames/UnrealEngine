// Copyright Epic Games, Inc. All Rights Reserved.

#include "SColorSlider.h"

#include "Widgets/Text/STextBlock.h"

SColorSlider::SColorSlider()
	: Orientation(*this, EOrientation::Orient_Horizontal)
	, GradientColors(*this)
	, bHasAlphaBackground(*this, false)
	, bUseSRGB(*this, false)
	, bSupportDynamicSliderMaxValue(*this, true)
{
}

void SColorSlider::Construct(const FArguments& InArgs)
{
	GradientColors.Assign(*this, InArgs._GradientColors);

	bHasAlphaBackground.Assign(*this, InArgs._HasAlphaBackground.Get());
	bUseSRGB.Assign(*this, InArgs._UseSRGB);
	bSupportDynamicSliderMaxValue.Assign(*this, InArgs._SupportDynamicSliderMaxValue);
	Orientation.Assign(*this, InArgs._Orientation.Get());

	BorderBrush = FAppStyle::Get().GetBrush("ColorPicker.RoundedInputBorder");
	BorderActiveBrush = FAppStyle::Get().GetBrush("ColorPicker.RoundedInputBorderActive");
	BorderHoveredBrush = FAppStyle::Get().GetBrush("ColorPicker.RoundedInputBorderHovered");
	AlphaBackgroundBrush = FAppStyle::Get().GetBrush("ColorPicker.RoundedAlphaBackground");

	const FSlateFontInfo SmallFont = FAppStyle::Get().GetFontStyle("ColorPicker.SmallFont");

	ColorSliderSize = (Orientation.Get() == Orient_Horizontal) ? HorizontalSliderLength : VerticalSliderHeight;

	SAssignNew(Slider, SSlider)
		.IndentHandle(false)
		.Orientation(Orientation.Get())
		.SliderBarColor(FLinearColor::Transparent)
		.SliderHandleColor(FLinearColor::Transparent)
		.Style(&FAppStyle::Get().GetWidgetStyle<FSliderStyle>("ColorPicker.Slider"))
		.MinValue(InArgs._MinSliderValue.Get())
		.MaxValue(InArgs._MaxSliderValue.Get())
		.StepSize(InArgs._Delta.Get())
		.Value(InArgs._Value)
		.OnMouseCaptureBegin(InArgs._OnBeginSliderMovement)
		.OnMouseCaptureEnd(InArgs._OnEndSliderMovement)
		.OnValueChanged(InArgs._OnValueChanged)
		.PreventThrottling(InArgs._PreventThrottling);

	TSharedPtr<SHorizontalBox> ColorWidget = SNew(SHorizontalBox);

	if (Orientation.Get() == Orient_Horizontal)
	{
		ColorWidget->AddSlot()
			.MinWidth(LabelSize)
			.MaxWidth(LabelSize)
			.Padding(0.0f, 0.0f, Padding, 0.0f)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text(InArgs._Label)
							.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallText"))
							.Justification(ETextJustify::Left)
					]
			];

		ColorWidget->AddSlot()
			.MinWidth(ColorSliderSize)
			.MaxWidth(ColorSliderSize)
			[
				Slider.ToSharedRef()
			];

		ColorWidget->AddSlot()
			.MinWidth(SpinBoxSize)
			.MaxWidth(SpinBoxSize)
			.Padding(Padding, 0.0f, 0.0f, 0.0f)
			[
				SNew(SSpinBox<float>)
					.Style(FAppStyle::Get(), "ColorSlider.SpinBox")
					.MinValue(InArgs._MinSpinBoxValue.Get())
					.MaxValue(InArgs._MaxSpinBoxValue.Get())
					.MinSliderValue(InArgs._MinSliderValue.Get())
					.MaxSliderValue(InArgs._MaxSliderValue.Get())
					.MaxFractionalDigits(3)
					.Delta(InArgs._Delta)
					.Value(InArgs._Value)
					.SupportDynamicSliderMaxValue(InArgs._SupportDynamicSliderMaxValue)
					.OnBeginSliderMovement(InArgs._OnBeginSpinBoxMovement)
					.OnEndSliderMovement(InArgs._OnEndSpinBoxMovement)
					.OnValueChanged(InArgs._OnValueChanged)
					.OnValueCommitted(this, &SColorSlider::OnSpinBoxValueCommitted)
					.Font(SmallFont)
			];
	}
	else
	{
		ColorWidget->AddSlot()
			[
				Slider.ToSharedRef()
			];
	}

	ChildSlot
	[
		ColorWidget.ToSharedRef()
	];
}

FVector2D SColorSlider::ComputeDesiredSize(float) const
{
	if (Orientation.Get() == EOrientation::Orient_Horizontal)
	{
		constexpr float TotalWidth = LabelSize + Padding + HorizontalSliderLength + Padding + SpinBoxSize;
		return FVector2D(TotalWidth, HorizontalSliderHeight);
	}
	else
	{
		return FVector2D(VerticalSliderWidth, VerticalSliderHeight);
	}
}

int32 SColorSlider::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// Render the underlying widgets underneath so we can draw the gradient, selector, and border on top
	const int32 MaxChildLayer = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	LayerId = MaxChildLayer + 1;

	const ESlateDrawEffect DrawEffects = ESlateDrawEffect::None;

	FVector2f SliderSize;
	FVector2f SliderOffset = FVector2f(0.0f, 0.0f);

	if (Orientation.Get() == Orient_Horizontal)
	{
		SliderSize = FVector2f(ColorSliderSize, AllottedGeometry.GetLocalSize().Y);
		SliderOffset.X = LabelSize + Padding;
	}
	else
	{
		SliderSize = FVector2f(AllottedGeometry.GetLocalSize().X, ColorSliderSize);
	}

	// Draw Color Gradient
	const int32 NumColors = GradientColors.Get().Num();

	if (NumColors > 1)
	{
		if (bHasAlphaBackground.Get())
		{
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId++,
				AllottedGeometry.ToPaintGeometry(SliderSize, FSlateLayoutTransform(SliderOffset)),
				AlphaBackgroundBrush,
				DrawEffects
			);
		}

		// Vertical sliders need the start color at the bottom and the end color at the bottom
		TArray<FLinearColor> Colors = GradientColors.Get();
		if (Orientation.Get() == Orient_Vertical)
		{
			Algo::Reverse(Colors);
		}

		// Generate the gradient stops for the colors
		TArray<FSlateGradientStop> GradientStops;
		for (int32 ColorIndex = 0; ColorIndex < NumColors; ++ColorIndex)
		{
			FColor DrawColor = Colors[ColorIndex].ToFColor(bUseSRGB.Get());
			GradientStops.Add(FSlateGradientStop(ColorSliderSize * (float(ColorIndex) / (NumColors - 1)), DrawColor));
		}

		const FVector4f CornerRadius = FVector4f(4.0f, 4.0f, 4.0f, 4.0f);

		FSlateDrawElement::MakeGradient(
			OutDrawElements,
			LayerId++,
			AllottedGeometry.ToPaintGeometry(SliderSize, FSlateLayoutTransform(SliderOffset)),
			GradientStops,
			(Orientation.Get() == Orient_Horizontal) ? EOrientation::Orient_Vertical : EOrientation::Orient_Horizontal,
			DrawEffects,
			CornerRadius
		);	
	}

	// Draw Selector
	const FSlateBrush* SelectorBrush;
	const FVector2f AllottedGeometrySize = AllottedGeometry.GetLocalSize();

	// If the current slider value is greater than the slider's maximum value, update the slider maximum.
	// This could occur because the maximum value of the spinbox might be greater than the maximum value of the slider. 
	const float SliderValue = Slider->GetValue();
	if ((SliderValue > Slider->GetMaxValue()) && bSupportDynamicSliderMaxValue.Get())
	{
		Slider->SetMinAndMaxValues(Slider->GetMinValue(), SliderValue);
	}

	float FractionFilled = SliderValue / Slider->GetMaxValue();

	FVector2f SelectorSize;
	FVector2f SelectorOffset;

	constexpr float SelectorThickness = 3.0f;

	if (Orientation.Get() == Orient_Horizontal)
	{
		SelectorBrush = FAppStyle::Get().GetBrush("ColorPicker.SpinBoxSelectorVertical");
		SelectorSize = FVector2f(SelectorThickness, HorizontalSliderHeight - 2.0f);

		float SelectorRange = ColorSliderSize - SelectorSize.X;
		SelectorOffset = FVector2f(SelectorRange * FractionFilled, 1.0f);
	}
	else
	{
		SelectorBrush = FAppStyle::Get().GetBrush("ColorPicker.SpinBoxSelectorHorizontal");
		SelectorSize = FVector2f(VerticalSliderWidth - 2.0f, SelectorThickness);

		// Invert so that 1 is the top of the slider and 0 is the bottom of the slider
		FractionFilled = 1.0f - FractionFilled;

		float SelectorRange = ColorSliderSize - SelectorSize.Y;
		SelectorOffset = FVector2f(1.0f, SelectorRange * FractionFilled);
	}

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId++,
		AllottedGeometry.ToPaintGeometry(SelectorSize, FSlateLayoutTransform(SelectorOffset + SliderOffset)),
		SelectorBrush,
		DrawEffects,
		SelectorBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
	);

	// Draw Border
	const FSlateBrush* BorderImage = Slider->HasMouseCapture() ? BorderActiveBrush : (Slider->IsHovered()) ? BorderHoveredBrush : BorderBrush;

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId++,
		AllottedGeometry.ToPaintGeometry(SliderSize, FSlateLayoutTransform(SliderOffset)),
		BorderImage,
		DrawEffects,
		BorderImage->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
	);

	return LayerId;
}

void SColorSlider::OnSpinBoxValueCommitted(float NewValue, ETextCommit::Type CommitType)
{
	if ((NewValue > Slider->GetMaxValue()) && bSupportDynamicSliderMaxValue.Get())
	{
		Slider->SetMinAndMaxValues(Slider->GetMinValue(), NewValue);
	}
}
