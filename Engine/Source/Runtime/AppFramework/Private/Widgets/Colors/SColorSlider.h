// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Styling/SlateTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Input/SSpinBox.h"

/**
 * Implements a color slider widget. 
 * When the orientation is horizontal, it features an STextBlock label, a SSlider with color gradient drawn on top, and a SSpinBox.
 * When the orientation is vertical, it features only the SSlider with color gradient
 */
class SColorSlider : public SCompoundWidget
{
public:
	SColorSlider();

	/** Notification for numeric value change */
	DECLARE_DELEGATE_OneParam(FOnValueChanged, float);

	SLATE_BEGIN_ARGS(SColorSlider)
		: _Value(0.0f)
		, _MinSpinBoxValue(0.0f)
		, _MaxSpinBoxValue(1.0f)
		, _MinSliderValue(0.0f)
		, _MaxSliderValue(1.0f)
		, _Delta(0.01f)
		, _SupportDynamicSliderMaxValue(true)
		, _Orientation(Orient_Horizontal)
		, _Label()
		, _GradientColors()
		, _HasAlphaBackground(false)
		, _UseSRGB(true)
		, _PreventThrottling(false)
		, _OnValueChanged()
		, _OnBeginSliderMovement()
		, _OnEndSliderMovement()
		, _OnBeginSpinBoxMovement()
		, _OnEndSpinBoxMovement()
	{ }

		/** The value that determines where the slider handle is drawn */
		SLATE_ATTRIBUTE(float, Value)

		/** The minimum value of the spinbox */
		SLATE_ATTRIBUTE(float, MinSpinBoxValue)

		/** The maximum value of the spinbox */
		SLATE_ATTRIBUTE(float, MaxSpinBoxValue)

		/** The minimum value of the slider */
		SLATE_ATTRIBUTE(float, MinSliderValue)

		/** The maximum value of the slider */
		SLATE_ATTRIBUTE(float, MaxSliderValue)

		/** The delta to increment the value as the slider moves */
		SLATE_ATTRIBUTE(float, Delta)

		/** Whether the underlying spinbox supports changing the maximum slider value */
		SLATE_ATTRIBUTE(bool, SupportDynamicSliderMaxValue)

		/** Orientation of the slider */
		SLATE_ATTRIBUTE(EOrientation, Orientation)

		/** Text content of the TextBlock (horizontal sliders only) */
		SLATE_ATTRIBUTE(FText, Label)

		/** List of colors which determine the gradient stops drawn on top of the slider */
		SLATE_ATTRIBUTE(TArray<FLinearColor>, GradientColors)

		/** Whether a checker background is displayed for alpha viewing */
		SLATE_ATTRIBUTE(bool, HasAlphaBackground)

		/** Whether to display sRGB color */
		SLATE_ATTRIBUTE(bool, UseSRGB)

		/** Whether to update the viewport with every single change to the value during dragging */
		SLATE_ARGUMENT(bool, PreventThrottling)

		/** Called when the value is changed by the Slider or SpinBox */
		SLATE_EVENT(FOnValueChanged, OnValueChanged)

		/** Called right before the slider handle on the Slider widget begins to move */
		SLATE_EVENT(FSimpleDelegate, OnBeginSliderMovement)

		/** Called right after the slider handle on the Slider widget is released by the user */
		SLATE_EVENT(FSimpleDelegate, OnEndSliderMovement)

		/** Called right before the slider handle on the SpinBox widget begins to move */
		SLATE_EVENT(FSimpleDelegate, OnBeginSpinBoxMovement)

		/** Called right after the slider handle on the SpinBox widget is released by the user */
		SLATE_EVENT(FOnValueChanged, OnEndSpinBoxMovement)

	SLATE_END_ARGS()

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	void Construct(const FArguments& InArgs);

protected:
	// Begin SWidget overrides
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FVector2D ComputeDesiredSize(float) const override;
	// End SWidget overrides

private:
	/** Update the max value of the slider if the value committed is higher than the current maximum slider value */
	void OnSpinBoxValueCommitted(float NewValue, ETextCommit::Type CommitType);

private:
	/** Orientation of the slider */
	TSlateAttribute<EOrientation, EInvalidateWidgetReason::Paint> Orientation;

	/** The colors used in the gradient. */
	TSlateAttribute<TArray<FLinearColor>, EInvalidateWidgetReason::Paint> GradientColors;

	/** Whether a checker background is displayed for alpha viewing */
	TSlateAttribute<bool, EInvalidateWidgetReason::Paint> bHasAlphaBackground;

	/** Whether to display sRGB color */
	TSlateAttribute<bool, EInvalidateWidgetReason::Paint> bUseSRGB;

	/** Whether to dynamically update the maximum slider value */
	TSlateAttribute<bool, EInvalidateWidgetReason::Paint> bSupportDynamicSliderMaxValue;

	/** Slider widget */
	TSharedPtr<SSlider> Slider;

	/** Widget layout sizes */
	float ColorSliderSize = 0.0f;

	static constexpr float Padding = 8.0f;
	static constexpr float LabelSize = 8.0f;
	static constexpr float SpinBoxSize = 60.0f;

	static constexpr float HorizontalSliderLength = 123.0f;
	static constexpr float HorizontalSliderHeight = 20.0f;

	static constexpr float VerticalSliderWidth = 28.0f;
	static constexpr float VerticalSliderHeight = 200.0f;

	/** Styling brushes */
	const FSlateBrush* BorderBrush;
	const FSlateBrush* BorderActiveBrush;
	const FSlateBrush* BorderHoveredBrush;
	const FSlateBrush* AlphaBackgroundBrush;
};
