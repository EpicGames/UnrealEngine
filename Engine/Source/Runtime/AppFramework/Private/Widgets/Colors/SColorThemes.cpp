// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Colors/SColorThemes.h"

#include "Algo/Find.h"
#include "Application/SlateWindowHelper.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Layout/ArrangedChildren.h"
#include "Misc/ConfigCacheIni.h"
#include "SlateOptMacros.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SToolTip.h"


#define LOCTEXT_NAMESPACE "SColorThemes"


void FColorDragDrop::OnDrop( bool bDropWasHandled, const FPointerEvent& MouseEvent )
{
	HideTrash.ExecuteIfBound();
	FDragDropOperation::OnDrop( bDropWasHandled, MouseEvent );
}

void FColorDragDrop::OnDragged( const class FDragDropEvent& DragDropEvent )
{
	if (CursorDecoratorWindow.IsValid())
	{
		CursorDecoratorWindow->MoveWindowTo(DragDropEvent.GetScreenSpacePosition() - (CursorDecoratorWindow->GetSizeInScreen() * 0.5f));
	}
}

TSharedRef<FColorDragDrop> FColorDragDrop::New(FLinearColor InColor, bool bSRGB, bool bUseAlpha,
	FSimpleDelegate TrashShowCallback, FSimpleDelegate TrashHideCallback,
	TSharedPtr<SThemeColorBlocksBar> Origin, int32 OriginPosition)
{
	TSharedRef<FColorDragDrop> Operation = MakeShareable(new FColorDragDrop(InColor, bSRGB, bUseAlpha, TrashShowCallback, TrashHideCallback));
	return Operation;
}

TSharedPtr<SWidget> FColorDragDrop::GetDefaultDecorator() const
{
	return SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("ColorPicker.MultipleValuesBackground"))
		.Padding(FMargin(1.0f, 1.0f))
		[
			SNew(SColorBlock) 
				.Color(Color)
				.ColorIsHSV(true) 
				.AlphaDisplayMode(bUseAlpha ? EColorBlockAlphaDisplayMode::SeparateReverse : EColorBlockAlphaDisplayMode::Ignore)
				.ShowBackgroundForAlpha(bUseAlpha)
				.UseSRGB(bUseSRGB)
				.Size(FVector2D(22.0, 22.0))
				.CornerRadius(FVector4(4.0, 4.0, 4.0, 4.0))
		];
}

FColorDragDrop::FColorDragDrop(FLinearColor InColor, bool bInUseSRGB, bool bInUseAlpha, FSimpleDelegate InTrashShowCallback, FSimpleDelegate InTrashHideCallback)
{
	Color = InColor;
	bUseSRGB = bInUseSRGB;
	bUseAlpha = bInUseAlpha;
	ShowTrash = InTrashShowCallback;
	HideTrash = InTrashHideCallback;

	TSharedPtr<SWidget> DecoratorToUse = GetDefaultDecorator();

	if (DecoratorToUse.IsValid())
	{
		CursorDecoratorWindow = SWindow::MakeStyledCursorDecorator(FAppStyle::Get().GetWidgetStyle<FWindowStyle>("ColorPicker.CursorDecorator"));
		CursorDecoratorWindow->SetContent(DecoratorToUse.ToSharedRef());

		FSlateApplicationBase::Get().AddWindow(CursorDecoratorWindow.ToSharedRef(), true);
	}

	ShowTrash.ExecuteIfBound();
}

void FColorDragDrop::MarkForDelete()
{
	CursorDecoratorWindow->SetOpacity(0.4f);
}

void FColorDragDrop::MarkForAdd()
{
	CursorDecoratorWindow->SetOpacity(1.0f);
}


FColorTheme::FColorTheme( const FString& InName, const TArray< TSharedPtr<FColorInfo> >& InColors )
	: Name(InName)
	, Colors(InColors)
	, RefreshEvent()
{ }

void FColorTheme::InsertNewColor(TSharedPtr<FColorInfo> InColor, int32 InsertPosition)
{
	Colors.Insert(InColor, InsertPosition);
	RefreshEvent.Broadcast();
}

void FColorTheme::InsertNewColor( TSharedPtr<FLinearColor> InColor, int32 InsertPosition )
{
	TSharedPtr<FColorInfo> NewColor = MakeShareable(new FColorInfo(InColor));
	Colors.Insert(NewColor, InsertPosition);
	RefreshEvent.Broadcast();
}

int32 FColorTheme::FindApproxColor( const FLinearColor& InColor, float Tolerance ) const
{
	for (int32 ColorIndex = 0; ColorIndex < Colors.Num(); ++ColorIndex)
	{
		TSharedPtr<FLinearColor> ApproxColor = Colors[ColorIndex]->Color;
		if (ApproxColor->Equals(InColor, Tolerance))
		{
			return ColorIndex;
		}
	}

	return INDEX_NONE;
}

void FColorTheme::RemoveAll()
{
	Colors.Empty();
	RefreshEvent.Broadcast();
}

int32 FColorTheme::RemoveColor( const TSharedPtr<FLinearColor> InColor )
{
	int32 Index = INDEX_NONE;
	const TSharedPtr<FColorInfo>* MatchingColor = Colors.FindByPredicate([InColor](TSharedPtr<FColorInfo>& ColorInfo) { return (ColorInfo->Color == InColor); });
	if (MatchingColor != nullptr)
	{
		Index = Colors.Find(*MatchingColor);
		Colors.RemoveAt(Index);
		RefreshEvent.Broadcast();
	}
	return Index;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void SColorTrash::Construct( const FArguments& InArgs )
{
	bBorderActivated = false;

	this->ChildSlot
	[
		SNew(SBorder)
			.ToolTipText(LOCTEXT("MouseOverToolTip", "Delete Color"))
			.BorderImage(this, &SColorTrash::GetBorderStyle)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot() .HAlign(HAlign_Center) .FillWidth(1)
			[
				SNew(SImage) 
				.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
			]
		]
	];
}
	
/**
* Called during drag and drop when the drag enters a widget.
*
* @param MyGeometry      The geometry of the widget receiving the event.
* @param DragDropEvent   The drag and drop event.
*
* @return A reply that indicated whether this event was handled.
*/
void SColorTrash::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	if ( DragDropEvent.GetOperationAs<FColorDragDrop>().IsValid() )
	{
		bBorderActivated = true;
	}
}

/**
* Called during drag and drop when the drag leaves a widget.
*
* @param DragDropEvent   The drag and drop event.
*
* @return A reply that indicated whether this event was handled.
*/
void SColorTrash::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	if ( DragDropEvent.GetOperationAs<FColorDragDrop>().IsValid() )
	{
		bBorderActivated = false;
	}
}

/**
* Called when the user is dropping something onto a widget; terminates drag and drop.
*
* @param MyGeometry      The geometry of the widget receiving the event.
* @param DragDropEvent   The drag and drop event.
*
* @return A reply that indicated whether this event was handled.
*/
FReply SColorTrash::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FColorDragDrop> DragDropContent = DragDropEvent.GetOperationAs<FColorDragDrop>();
	if ( DragDropContent.IsValid() )
	{
		DragDropContent->bSetForDeletion = true;

		bBorderActivated = false;

		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

const FSlateBrush* SColorTrash::GetBorderStyle() const
{
	return (bBorderActivated) ?
		FAppStyle::Get().GetBrush("FocusRectangle") :
		FStyleDefaults::GetNoBrush();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/**
* Construct the widget
*
* @param InArgs   Declaration from which to construct the widget.
*/
void SThemeColorBlock::Construct(const FArguments& InArgs )
{
	ColorPtr = InArgs._Color.Get();
	ColorInfo = InArgs._ColorInfo.Get();
	OnSelectColor = InArgs._OnSelectColor;
	ParentPtr = InArgs._Parent.Get();
	ShowTrashCallback = InArgs._ShowTrashCallback;
	HideTrashCallback = InArgs._HideTrashCallback;
	bUseSRGB = InArgs._UseSRGB;
	bUseAlpha = InArgs._UseAlpha;
	bSupportsDrag = InArgs._SupportsDrag;

	const FSlateFontInfo SmallLayoutFont = FAppStyle::Get().GetFontStyle("Regular");
	const FSlateFontInfo SmallLabelFont = FAppStyle::Get().GetFontStyle("Bold");

	TSharedPtr<SToolTip> ColorTooltip =
		SNew(SToolTip)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot().AutoWidth()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot().AutoHeight() .Padding(2)
				[
					SNew(SBox)
					.WidthOverride(110)
					.HeightOverride(110)
					[
						SNew(SColorBlock)
							.Color(this, &SThemeColorBlock::GetColor)
							.ColorIsHSV(true)
							.AlphaDisplayMode(this, &SThemeColorBlock::OnGetAlphaDisplayMode)
							.ShowBackgroundForAlpha(this, &SThemeColorBlock::OnReadShowBackgroundForAlpha)
							.UseSRGB(bUseSRGB)
					]
				]
				+ SVerticalBox::Slot().AutoHeight().Padding(2)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock).Font(SmallLabelFont).Text(this, &SThemeColorBlock::GetLabel) .Visibility(SharedThis(this), &SThemeColorBlock::OnGetLabelVisibility)
				]
				+SVerticalBox::Slot().AutoHeight() .Padding(2)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot() .FillWidth(1)
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot().AutoHeight() .Padding(3)
						[
							SNew(STextBlock) .Font(SmallLayoutFont) .Text(this, &SThemeColorBlock::GetRedText)
						]
						+SVerticalBox::Slot().AutoHeight() .Padding(3)
						[
							SNew(STextBlock) .Font(SmallLayoutFont) .Text(this, &SThemeColorBlock::GetGreenText)
						]
						+SVerticalBox::Slot().AutoHeight() .Padding(3)
						[
							SNew(STextBlock) .Font(SmallLayoutFont) .Text(this, &SThemeColorBlock::GetBlueText)
						]
					]
					+SHorizontalBox::Slot() .FillWidth(1)
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot().AutoHeight() .Padding(3)
						[
							SNew(STextBlock) .Font(SmallLayoutFont) .Text(this, &SThemeColorBlock::GetHueText)
						]
						+SVerticalBox::Slot().AutoHeight() .Padding(3)
						[
							SNew(STextBlock) .Font(SmallLayoutFont) .Text(this, &SThemeColorBlock::GetSaturationText)
						]
						+SVerticalBox::Slot().AutoHeight() .Padding(3)
						[
							SNew(STextBlock) .Font(SmallLayoutFont) .Text(this, &SThemeColorBlock::GetValueText)
						]
					]
				]
				+SVerticalBox::Slot().AutoHeight() .Padding(2)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock) .Font(SmallLayoutFont) .Text(this, &SThemeColorBlock::GetAlphaText) .Visibility(SharedThis(this), &SThemeColorBlock::OnGetAlphaVisibility)
				]
			]
		];

	this->ChildSlot
	[
		SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ColorPicker.MultipleValuesBackground"))
			.Padding(FMargin(1.0f))
			.ToolTip(ColorTooltip)
			[
				SNew(SColorBlock)
				.Color(this, &SThemeColorBlock::GetColor)
				.AlphaDisplayMode(bUseAlpha.Get() ? EColorBlockAlphaDisplayMode::SeparateReverse : EColorBlockAlphaDisplayMode::Ignore)
				.ColorIsHSV(true)
				.ShowBackgroundForAlpha(bUseAlpha.Get())
				.UseSRGB(bUseSRGB)
				.Size(FVector2D(22.0, 22.0))
				.CornerRadius(FVector4(4.0, 4.0, 4.0, 4.0))
			]
	];
}

void SThemeColorBlock::OnColorBlockRename()
{
	// Field to enter new color label
	TSharedRef<STextEntryPopup> TextEntry =
		SNew(STextEntryPopup)
		.Label(LOCTEXT("NewColorLabel", "Color Label"))
		.OnTextCommitted(this, &SThemeColorBlock::SetLabel);

	// Show dialog to enter new color label
	FSlateApplication::Get().PushMenu(
		AsShared(), 
		FWidgetPath(),
		TextEntry,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
	);
}


FText SThemeColorBlock::GetLabel() const
{
	return ColorInfo->Label;
}


void SThemeColorBlock::SetLabel(const FText& NewColorLabel, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		ColorInfo->Label = NewColorLabel;
	}
	FSlateApplication::Get().DismissAllMenus();
	SColorThemesViewer::SaveColorThemesToIni();
}

FReply SThemeColorBlock::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && bSupportsDrag.Get())
	{
		return FReply::Handled().DetectDrag( SharedThis(this), EKeys::LeftMouseButton ).CaptureMouse(SharedThis(this));
	}
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		OnColorBlockRename();
		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SThemeColorBlock::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && MyGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition()) )
	{
		check(ColorPtr.IsValid());
		OnSelectColor.ExecuteIfBound(GetColor());
		
		return FReply::Handled().ReleaseMouseCapture();
			
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SThemeColorBlock::OnDragDetected( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton ) && bSupportsDrag.Get())
	{
		if (ParentPtr.IsValid())
		{
			ParentPtr.Pin()->RemoveColorBlock(ColorPtr.Pin());
		}

		TSharedRef<FColorDragDrop> Operation = FColorDragDrop::New(GetColor(), bUseSRGB.Get(), bUseAlpha.Get(), ShowTrashCallback, HideTrashCallback);
		return FReply::Handled().BeginDragDrop(Operation);
	}
	
	return FReply::Unhandled();
}

FLinearColor SThemeColorBlock::GetColor() const
{
	return ColorPtr.IsValid() ? *ColorPtr.Pin() : FLinearColor(ForceInit);
}

FText SThemeColorBlock::GetRedText() const { return ColorPtr.IsValid() ? FormatToolTipText(LOCTEXT("Red", "R"), ColorPtr.Pin()->HSVToLinearRGB().R) : FText::GetEmpty(); }
FText SThemeColorBlock::GetGreenText() const { return ColorPtr.IsValid() ? FormatToolTipText(LOCTEXT("Green", "G"), ColorPtr.Pin()->HSVToLinearRGB().G) : FText::GetEmpty(); }
FText SThemeColorBlock::GetBlueText() const { return ColorPtr.IsValid() ? FormatToolTipText(LOCTEXT("Blue", "B"), ColorPtr.Pin()->HSVToLinearRGB().B) : FText::GetEmpty(); }
FText SThemeColorBlock::GetAlphaText() const { return ColorPtr.IsValid() ? FormatToolTipText(LOCTEXT("Alpha", "A"), ColorPtr.Pin()->HSVToLinearRGB().A) : FText::GetEmpty(); }
FText SThemeColorBlock::GetHueText() const { return ColorPtr.IsValid() ? FormatToolTipText(LOCTEXT("Hue", "H"), FMath::RoundToFloat(ColorPtr.Pin()->R)) : FText::GetEmpty(); }	// Rounded to let the value match the value in the Hue spinbox in the color picker
FText SThemeColorBlock::GetSaturationText() const { return ColorPtr.IsValid() ? FormatToolTipText(LOCTEXT("Saturation", "S"), ColorPtr.Pin()->G) : FText::GetEmpty(); }
FText SThemeColorBlock::GetValueText() const { return ColorPtr.IsValid() ? FormatToolTipText(LOCTEXT("Value", "V"), ColorPtr.Pin()->B) : FText::GetEmpty(); }

FText SThemeColorBlock::FormatToolTipText(const FText& ColorIdentifier, float Value) const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("Identifier"), ColorIdentifier);

	if (Value >= 0.f)
	{
		static const float LogToLog10 = 1.f / FMath::Loge(10.f);
		int32 PreRadixDigits = FMath::Max(0, int32(FMath::Loge(Value + KINDA_SMALL_NUMBER) * LogToLog10));

		int32 Precision = FMath::Max(0, 2 - PreRadixDigits);

		FNumberFormattingOptions FormatRules;
		FormatRules.MinimumFractionalDigits = Precision;

		Args.Add(TEXT("Value"), FText::AsNumber(Value, &FormatRules));
	}
	else
	{
		Args.Add(TEXT("Value"), FText::GetEmpty());
	}

	return FText::Format(LOCTEXT("ToolTipFormat", "{Identifier}: {Value}"), Args);
}

EColorBlockAlphaDisplayMode SThemeColorBlock::OnGetAlphaDisplayMode() const
{
	return !bUseAlpha.Get() ? EColorBlockAlphaDisplayMode::Ignore : EColorBlockAlphaDisplayMode::Combined;
}

bool SThemeColorBlock::OnReadShowBackgroundForAlpha() const
{
	return bUseAlpha.Get();
}

EVisibility SThemeColorBlock::OnGetAlphaVisibility() const
{
	return bUseAlpha.Get() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SThemeColorBlock::OnGetLabelVisibility() const
{
	return !ColorInfo->Label.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed;
}

SThemeColorBlocksBar::SThemeColorBlocksBar()
	: Children(this)
{
}

void SThemeColorBlocksBar::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	bool bPlaceholderExists = PlaceholderIndex.IsSet();

	const FVector2D BlockSize = FVector2D(24.0f, 24.0f);
	constexpr float Padding = 2.0f;

	// Add the combo button, which takes up two grid slots (with padding)
	ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(ThemesViewer.ToSharedRef(), FVector2D(0.0, 0.0), FVector2D(50.0, 24.0)));
	int32 OccupiedGridSlots = 2;

	// Add the add/delete button, which takes up one grid slot
	if (!ThemesViewer->IsRecentsThemeActive())
	{
		ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(AddDeleteOverlay.ToSharedRef(), FVector2D(52.0, 0.0), BlockSize));
		OccupiedGridSlots++;
	}

	const int32 NumColorBlocks = bPlaceholderExists ? ColorBlocks.Num() + 1 : ColorBlocks.Num();
	const int32 NumGridBlocks = NumColorBlocks + OccupiedGridSlots;

	int32 ColorIndex = 0;
	for (int32 GridIndex = OccupiedGridSlots; GridIndex < NumGridBlocks; ++GridIndex)
	{
		const float HPadding = (GridIndex % 16) ? Padding : 0.0f;
		const float VPadding = (GridIndex / 16) ? Padding : 0.0f;

		float YOffset = (GridIndex / 16) * (BlockSize.Y + Padding);
		float XOffset = (GridIndex % 16) * (BlockSize.X + Padding);
		FVector2D DragShadowLocation = FVector2D(XOffset, YOffset);

		if (PlaceholderIndex.IsSet() && (GridIndex == PlaceholderIndex))
		{
			ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(DragShadow.ToSharedRef(), DragShadowLocation, BlockSize));
		}
		else if (ColorIndex < ColorBlocks.Num())
		{
			ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(ColorBlocks[ColorIndex].ToSharedRef(), FVector2D(XOffset, YOffset), BlockSize));
			ColorIndex++;
		}
	}
}

FVector2D SThemeColorBlocksBar::ComputeDesiredSize( float ) const
{
	const bool bPlaceholderExists = PlaceholderIndex.IsSet();
	const bool bAddDeleteButtonVisible = !ThemesViewer->IsRecentsThemeActive();
	const int32 NumColorBlocks = bPlaceholderExists ? ColorBlocks.Num() + 1 : ColorBlocks.Num();
	const int32 NumGridBlocks = bAddDeleteButtonVisible ? NumColorBlocks + 3 : NumColorBlocks + 2;

	const FVector2D BlockSize = FVector2D(24.0f, 24.0f);
	constexpr float Padding = 2.0f;

	int32 NumColorRows = ((NumGridBlocks - 1) / 16) + 1;
	float SizeY = (NumColorRows * (BlockSize.Y + Padding)) - Padding;

	float SizeX = (16 * (BlockSize.X + Padding)) - Padding;

	return FVector2D(SizeX, SizeY);
}

FChildren* SThemeColorBlocksBar::GetChildren()
{
	return &Children;
}

void SThemeColorBlocksBar::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FColorDragDrop> DragDropOperation = DragDropEvent.GetOperationAs<FColorDragDrop>();
	if ( DragDropOperation.IsValid() )
	{
		DragDropOperation->MarkForAdd();
	}
}

void SThemeColorBlocksBar::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FColorDragDrop> DragDropOperation = DragDropEvent.GetOperationAs<FColorDragDrop>();
	if ( DragDropOperation.IsValid() )
	{
		DragDropOperation->MarkForDelete();
		PlaceholderIndex.Reset();
	}
}

FReply SThemeColorBlocksBar::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FColorDragDrop> DragDropOperation = DragDropEvent.GetOperationAs<FColorDragDrop>();
	if ( DragDropOperation.IsValid() )
	{
		const FVector2D DragLocation = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());

		constexpr int32 BlockSizeWithPadding = 26;
		const int32 GridX = DragLocation.X / BlockSizeWithPadding;
		const int32 GridY = DragLocation.Y / BlockSizeWithPadding;

		if ((GridY == 0) && (GridX < 2))
		{
			// The dragged block is over the combo button
			DragDropOperation->MarkForAdd();
			PlaceholderIndex = 3;
		}
		else if ((GridY == 0) && (GridX == 2))
		{
			// The dragged block is over the delete button
			DragDropOperation->MarkForDelete();
			PlaceholderIndex.Reset();
		}
		else
		{
			DragDropOperation->MarkForAdd();
			const int32 NewPlaceholderIndex = (GridY * 16) + GridX;
			PlaceholderIndex = FMath::Clamp(NewPlaceholderIndex, 3, ColorBlocks.Num() + 3);
		}
			
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SThemeColorBlocksBar::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FColorDragDrop> DragDropOperation = DragDropEvent.GetOperationAs<FColorDragDrop>();
	if ( DragDropOperation.IsValid() )
	{
		// The combo button takes up two blocks, and the add/delete button takes up the third
		if (PlaceholderIndex.IsSet() && PlaceholderIndex.GetValue() >= 3)
		{
			const int32 AddIndex = PlaceholderIndex.GetValue() - 3;
			AddNewColorBlock(DragDropOperation->Color, AddIndex, true);
		}

		PlaceholderIndex.Reset();
		DragDropOperation->MarkForDelete();

		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}
}

void SThemeColorBlocksBar::AddNewColorBlock(FLinearColor Color, int32 InsertPosition, bool bAllowRepeat)
{
	// Do not add new colors to recents
	if (ColorTheme == ThemesViewer->GetRecents())
	{
		return;
	}

	// Check if the color being added is the same as the last one added
	bool bIsRepeatColor = false;
	const TArray<TSharedPtr<FColorInfo>>& Colors = ColorTheme->GetColors();
	if (Colors.Num() > 0)
	{
		TSharedPtr<FColorInfo> NewestColor = Colors[0];
		if (NewestColor->Color->Equals(Color))
		{
			bIsRepeatColor = true;
		}
	}

	if (!bIsRepeatColor || bAllowRepeat)
	{
		ColorTheme->InsertNewColor(MakeShareable(new FLinearColor(Color)), InsertPosition);
		SColorThemesViewer::SaveColorThemesToIni();
	}
}

void SThemeColorBlocksBar::AddToRecents(FLinearColor Color)
{
	if (TSharedPtr<FColorTheme> Recents = ThemesViewer->GetRecents())
	{
		// When the recents theme is active, the first row will have 14 color blocks, and each subsequent row will have 16
		constexpr int32 MaxNumRecentRows = 3;
		constexpr int32 MaxNumRecentColors = ((MaxNumRecentRows - 1) * 16) + 14;

		// If the recents theme is full, remove the oldest color before adding the new one
		if (Recents->GetColors().Num() == MaxNumRecentColors)
		{
			TSharedPtr<FColorInfo> OldestColor = Recents->GetColors().Last();
			Recents->RemoveColor(OldestColor->Color);
		}

		Recents->InsertNewColor(MakeShareable(new FLinearColor(Color)), 0);

		SColorThemesViewer::SaveColorThemesToIni();
	}
}

bool SThemeColorBlocksBar::IsRecentsThemeActive() const 
{
	return ThemesViewer->IsRecentsThemeActive();
}

int32 SThemeColorBlocksBar::RemoveColorBlock(TSharedPtr<FLinearColor> ColorToRemove)
{
	int32 Position = ColorTheme->RemoveColor(ColorToRemove);
	
	SColorThemesViewer::SaveColorThemesToIni();

	return Position;
}

void SThemeColorBlocksBar::RemoveRefreshCallback()
{
	// Deprecated function
}

void SThemeColorBlocksBar::AddRefreshCallback()
{
	// Deprecated function
}

void SThemeColorBlocksBar::OnThemeChanged()
{
	// Remove the refresh callback from the old theme
	ColorTheme->OnRefresh().Remove(RefreshCallbackHandle);

	// Get the new active theme and add our refresh callback to it so we can update the UI when the theme colors change
	ColorTheme = ThemesViewer->GetCurrentColorTheme();
	RefreshCallbackHandle = ColorTheme->OnRefresh().Add(RefreshCallback);

	Refresh();
}

FReply SThemeColorBlocksBar::OnAddButtonClicked()
{
	if (OnGetActiveColor.IsBound())
	{
		const FLinearColor ActiveColor = OnGetActiveColor.Execute();
		AddNewColorBlock(ActiveColor, 0);
	}
	return FReply::Handled();
}

void SThemeColorBlocksBar::ShowDeleteButton()
{
	bShowDeleteButton = true;
}

void SThemeColorBlocksBar::HideDeleteButton()
{
	bShowDeleteButton = false;
}

void SThemeColorBlocksBar::Refresh()
{
	Children.Empty();

	Children.Add(ThemesViewer.ToSharedRef());
	Children.Add(AddDeleteOverlay.ToSharedRef());

	ColorBlocks.Empty();
	check(ColorTheme.IsValid());

	const TArray< TSharedPtr<FColorInfo> >& Theme = ColorTheme->GetColors();
	for (int32 ColorIndex = 0; ColorIndex < Theme.Num(); ++ColorIndex)
	{
		const bool bSupportsDrag = (ColorTheme != ThemesViewer->GetRecents());

		ColorBlocks.Add(
			SNew(SThemeColorBlock)
				.Color(Theme[ColorIndex]->Color)
				.ColorInfo(Theme[ColorIndex])
				.OnSelectColor(OnSelectColor)
				.Parent(SharedThis(this))
				.ShowTrashCallback(this, &SThemeColorBlocksBar::ShowDeleteButton)
				.HideTrashCallback(this, &SThemeColorBlocksBar::HideDeleteButton)
				.UseSRGB(bUseSRGB)
				.UseAlpha(bUseAlpha)
				.SupportsDrag(bSupportsDrag)
			);

		Children.Add(ColorBlocks.Last().ToSharedRef());
	}
}

void SThemeColorBlocksBar::SetPlaceholderGrabOffset(FVector2D GrabOffset)
{
	// Deprecated function
}

EVisibility SThemeColorBlocksBar::GetAddButtonVisibility() const
{
	return bShowDeleteButton ? EVisibility::Hidden : EVisibility::Visible;
}

EVisibility SThemeColorBlocksBar::GetDeleteButtonVisibility() const
{
	return bShowDeleteButton ? EVisibility::Visible : EVisibility::Hidden;
}

void SThemeColorBlocksBar::Construct(const FArguments& InArgs)
{
	OnSelectColor = InArgs._OnSelectColor;
	OnGetActiveColor = InArgs._OnGetActiveColor;
	bUseSRGB = InArgs._UseSRGB;
	bUseAlpha = InArgs._UseAlpha;
	
	ThemesViewer = SNew(SColorThemesViewer);
	ThemesViewer->OnCurrentThemeChanged().AddSP(this, &SThemeColorBlocksBar::OnThemeChanged);
	
	RefreshCallback = FSimpleDelegate::CreateSP(this, &SThemeColorBlocksBar::Refresh);

	ColorTheme = ThemesViewer->GetCurrentColorTheme();
	RefreshCallbackHandle = ColorTheme->OnRefresh().Add(RefreshCallback);

	DragShadow = SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("ColorPicker.RoundedInputBorderHovered"))
		.Padding(FMargin(1.0f, 1.0f))
		[
			SNew(SBox)
				.WidthOverride(22.0f)
				.HeightOverride(22.0f)
				[
					SNullWidget::NullWidget
				]
		];

	AddDeleteOverlay = SNew(SOverlay)
		+ SOverlay::Slot()
		[
			SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("ColorPicker.AddButton"))
				.ContentPadding(FMargin(4.0))
				.Visibility(this, &SThemeColorBlocksBar::GetAddButtonVisibility)
				.ToolTipText(LOCTEXT("AddToThemeTooltip", "Add the currently selected color to the current color theme"))
				.OnClicked(this, &SThemeColorBlocksBar::OnAddButtonClicked)
				.Content()
				[
					SNew(SImage).Image(FAppStyle::Get().GetBrush("Icons.Plus"))
				]
		]

		+ SOverlay::Slot()
		[
			SNew(SButton)
				.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("ColorPicker.DeleteButton"))
				.ContentPadding(FMargin(4.0))
				.Visibility(this, &SThemeColorBlocksBar::GetDeleteButtonVisibility)
				.ToolTipText(LOCTEXT("DeleteFromThemeTooltip", "Delete this color from the current color theme"))
				.Content()
				[
					SNew(SImage).Image(FAppStyle::Get().GetBrush("Icons.Delete"))
				]
		];

	Refresh();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void SColorThemeBar::Construct(const FArguments& InArgs)
{
	ColorTheme = InArgs._ColorTheme.Get();
	OnCurrentThemeChanged = InArgs._OnCurrentThemeChanged;
	ShowTrashCallback = InArgs._ShowTrashCallback;
	HideTrashCallback = InArgs._HideTrashCallback;
	bUseSRGB = InArgs._UseSRGB;
	bUseAlpha = InArgs._UseAlpha;

	this->ChildSlot
	[
		SNew(SBox)
			.Padding(5.0f)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SAssignNew(ThemeNameText, STextBlock)
							.Text(this, &SColorThemeBar::GetThemeName)
							.Font(FCoreStyle::GetDefaultFontStyle("Regular", 10))
					]

				+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.Padding(8.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SThemeColorBlocksBar)
							.ColorTheme(InArgs._ColorTheme)
							.ShowTrashCallback(ShowTrashCallback)
							.HideTrashCallback(HideTrashCallback)
							.EmptyText(LOCTEXT("NoColorsText", "No Colors Added Yet"))
							.UseSRGB(bUseSRGB)
							.UseAlpha(bUseAlpha)
					]
			]
	];
}

FText SColorThemeBar::GetThemeName() const
{
	return FText::FromString(ColorTheme.Pin()->Name);
}

FReply SColorThemeBar::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		OnCurrentThemeChanged.ExecuteIfBound(ColorTheme.Pin());

		return FReply::Handled();
	}

	return FReply::Unhandled();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

TArray< TSharedPtr<FColorTheme> > SColorThemesViewer::ColorThemes;
TSharedPtr<FColorTheme> SColorThemesViewer::Recents;
TWeakPtr<FColorTheme> SColorThemesViewer::CurrentlySelectedThemePtr;
bool SColorThemesViewer::bSRGBEnabled = true;


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SColorThemesViewer::Construct(const FArguments& InArgs)
{
	if (!Recents.IsValid())
	{
		Recents = MakeShared<FColorTheme>();
	}

	LoadColorThemesFromIni();

	if (!CurrentlySelectedThemePtr.IsValid())
	{
		CurrentlySelectedThemePtr = Recents;
	}

	SAssignNew(RenameTextBox, SEditableTextBox)
		.IsEnabled(false)
		.OnTextCommitted(this, &SColorThemesViewer::CommitThemeName)
		.ClearKeyboardFocusOnCommit(false)
		.SelectAllTextWhenFocused(true)
		.MaximumLength(128);

	MultiBoxWidget = SNew(SMultiBoxWidget);
	RefreshMenuWidget();

	ChildSlot
		[
			SNew(SComboButton)
				.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("ColorPicker.ThemesComboButton"))
				.OnMenuOpenChanged(this, &SColorThemesViewer::OnMenuOpenChanged)
				.ToolTipText(LOCTEXT("ColorThemeComboButtonToolTip", "Color Theme Options"))
				.VAlign(VAlign_Center)
				.MenuContent()
				[
					MultiBoxWidget.ToSharedRef()
				]
				.ButtonContent()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
						.MinWidth(16)
						.MaxWidth(16)
						[
							SNew(SImage).Image(this, &SColorThemesViewer::GetComboButtonImage)
						]
				]
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SColorThemesViewer::MenuToStandardNoReturn()
{
	// Deprecated function
}

TSharedPtr<FColorTheme> SColorThemesViewer::GetCurrentColorTheme() const
{
	return CurrentlySelectedThemePtr.IsValid() ? CurrentlySelectedThemePtr.Pin() : ColorThemes[0];
}

TSharedPtr<FColorTheme> SColorThemesViewer::GetRecents() const
{
	return Recents;
}

const FSlateBrush* SColorThemesViewer::GetComboButtonImage() const
{
	return IsRecentsThemeActive() ? FAppStyle::Get().GetBrush("Icons.Recent") : FAppStyle::Get().GetBrush("ColorPicker.ColorThemesSmall");
};

bool SColorThemesViewer::IsRecentsThemeActive() const
{
	return (CurrentlySelectedThemePtr == Recents);
}

void SColorThemesViewer::OnMenuOpenChanged(bool bIsOpen)
{
	if (!bIsOpen)
	{
		StopRename();
	}
}

void SColorThemesViewer::BuildMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("RecentsSection");
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("RecentsTheme", "Recents"),
			LOCTEXT("RecentsThemeToolTip", "Recently used colors"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Recent"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SColorThemesViewer::SetCurrentColorTheme, Recents),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SColorThemesViewer::IsRecentsThemeActive)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("SavedThemes", LOCTEXT("SavedThemes", "Saved Color Themes"));
	{
		for (const TSharedPtr<FColorTheme>& ColorTheme : ColorThemes)
		{
			// Build this menu entry manually so that the rename widget can be added if enabled
			FMenuEntryParams MenuEntryParams;
			MenuEntryParams.LabelOverride = FText::FromString(ColorTheme->Name);
			MenuEntryParams.ToolTipOverride = FText::FromString(ColorTheme->Name);
			MenuEntryParams.IconOverride = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ColorPicker.ColorThemesSmall");
			MenuEntryParams.UserInterfaceActionType = EUserInterfaceActionType::RadioButton;
			MenuEntryParams.DirectActions = FUIAction(
				FExecuteAction::CreateSP(this, &SColorThemesViewer::SetCurrentColorTheme, ColorTheme),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([ColorTheme] { return (CurrentlySelectedThemePtr == ColorTheme); })
			);
			MenuEntryParams.ExtensionHook = NAME_None;

			if (CurrentlySelectedThemePtr == ColorTheme && RenameTextBox->IsEnabled())
			{
				RenameTextBox->SetText(FText::FromString(ColorTheme->Name));
				MenuEntryParams.EntryWidget = RenameTextBox.ToSharedRef();
			}

			MenuBuilder.AddMenuEntry(MenuEntryParams);
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.AddMenuSeparator();

	MenuBuilder.BeginSection("AddThemeSection");
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CreateNewTheme", "Create New Theme"),
			LOCTEXT("CreateNewThemeTooltip", "Create New Theme"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.PlusCircle"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SColorThemesViewer::NewColorTheme),
				FCanExecuteAction()
			),
			NAME_None,
			EUserInterfaceActionType::CollapsedButton
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("EditCurrentThemeSection", LOCTEXT("EditThemeSection", "Edit Current Theme"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("RenameTheme", "Rename"),
			LOCTEXT("RenameThemeToolTip", "Rename the currently selected color theme"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Rename"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SColorThemesViewer::StartRename),
				FCanExecuteAction::CreateLambda([this]() { return !IsRecentsThemeActive(); })
			),
			NAME_None,
			EUserInterfaceActionType::CollapsedButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("DuplicateTheme", "Duplicate"),
			LOCTEXT("DuplicateThemeTooltip", "Duplicate the currently selected color theme"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Duplicate"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SColorThemesViewer::DuplicateColorTheme),
				FCanExecuteAction::CreateLambda([this]() { return !IsRecentsThemeActive(); })
			),
			NAME_None,
			EUserInterfaceActionType::CollapsedButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("DeleteTheme", "Delete"),
			LOCTEXT("DeleteThemeTooltip", "Delete the currently selected color theme"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SColorThemesViewer::DeleteColorTheme),
				FCanExecuteAction::CreateLambda([this]() { return !IsRecentsThemeActive(); })
			),
			NAME_None,
			EUserInterfaceActionType::CollapsedButton
		);
	}
	MenuBuilder.EndSection();
}

void SColorThemesViewer::RefreshMenuWidget()
{
	FMenuBuilder MenuBuilder(false, nullptr);
	BuildMenu(MenuBuilder);
	MultiBoxWidget->UpdateMultiBoxWidget(MenuBuilder.GetMultiBox());
}

void SColorThemesViewer::StartRename()
{
	RenameTextBox->SetEnabled(true);
	RefreshMenuWidget();
	FSlateApplication::Get().SetKeyboardFocus(RenameTextBox.ToSharedRef());
}

void SColorThemesViewer::StopRename()
{
	RenameTextBox->SetEnabled(false);
	RefreshMenuWidget();
}

void SColorThemesViewer::SetUseAlpha( const TAttribute<bool>& InUseAlpha )
{
}

void SColorThemesViewer::SetCurrentColorTheme(TSharedPtr<FColorTheme> NewTheme)
{
	// Set the current theme, requires a preexisting theme to be passed in
	CurrentlySelectedThemePtr = NewTheme;
	CurrentThemeChangedEvent.Broadcast();

	if (FPaths::FileExists(GEditorPerProjectIni) && CurrentlySelectedThemePtr.IsValid())
	{
		GConfig->SetString(TEXT("ColorPickerUI"), TEXT("CurrentTheme"), *CurrentlySelectedThemePtr.Pin()->Name, GEditorPerProjectIni);
	}

	StopRename();
}

TSharedPtr<FColorTheme> SColorThemesViewer::IsColorTheme(const FString& ThemeName)
{
	// Find the desired theme
	for (int32 ThemeIndex = 0; ThemeIndex < ColorThemes.Num(); ++ThemeIndex)
	{
		const TSharedPtr<FColorTheme>& ColorTheme = ColorThemes[ThemeIndex];
		if ( ColorTheme->Name == ThemeName )
		{
			return ColorTheme;
		}
	}
	return TSharedPtr<FColorTheme>();
}

TSharedPtr<FColorTheme> SColorThemesViewer::GetColorTheme(const FString& ThemeName)
{
	// Create the desired theme, if not already
	TSharedPtr<FColorTheme> ColorTheme = IsColorTheme(ThemeName);
	if ( !ColorTheme.IsValid() )
	{
		ColorTheme = NewColorTheme(ThemeName);
	}
	return ColorTheme;
}

FString SColorThemesViewer::MakeUniqueThemeName(const FString& ThemeName)
{
	// Ensure the name of the color theme is unique
	int32 ThemeID = 1;
	FString NewThemeName = ThemeName;
	while ( IsColorTheme(NewThemeName).IsValid() )
	{
		NewThemeName = ThemeName + FString::Printf( TEXT( " %d" ), ThemeID );
		ThemeID++;
	}
	return NewThemeName;
}

TSharedPtr<FColorTheme> SColorThemesViewer::NewColorTheme(const FString& ThemeName, const TArray< TSharedPtr<FColorInfo> >& ThemeColors)
{
	// Create a uniquely named theme
	check( ThemeName.Len() > 0 );
	const FString NewThemeName = MakeUniqueThemeName( ThemeName );
	ColorThemes.Add(MakeShareable(new FColorTheme( NewThemeName, ThemeColors )));
	return ColorThemes.Last();
}

TSharedPtr<FColorTheme> SColorThemesViewer::GetDefaultColorTheme(bool bCreateNew)
{
	// Create a default theme (if bCreateNew is always creates a new one, even if there's already a like named theme)
	const FText Name = LOCTEXT("NewThemeName", "New Theme");
	if ( bCreateNew )
	{
		return NewColorTheme( Name.ToString() );
	}
	return GetColorTheme( Name.ToString() );
}

void SColorThemesViewer::CommitThemeName(const FText& InText, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnEnter)
	{
		UpdateThemeNameFromTextBox();
		StopRename();
	}
}

void SColorThemesViewer::UpdateThemeNameFromTextBox()
{
	// Update the theme name if it differs, ensuring it is still unique
	const FString Name = RenameTextBox->GetText().ToString();
	if (GetCurrentColorTheme()->Name != Name)
	{
		GetCurrentColorTheme()->Name = MakeUniqueThemeName(Name);
		SaveColorThemesToIni();

		if (FPaths::FileExists(GEditorPerProjectIni) && CurrentlySelectedThemePtr.IsValid())
		{
			GConfig->SetString(TEXT("ColorPickerUI"), TEXT("CurrentTheme"), *CurrentlySelectedThemePtr.Pin()->Name, GEditorPerProjectIni);
		}
	}
}

void SColorThemesViewer::NewColorTheme()
{
	// Create a new, defaultly named theme and update the display
	TSharedPtr<FColorTheme> NewTheme = GetDefaultColorTheme(true);
	SaveColorThemesToIni();

	SetCurrentColorTheme(NewTheme);
	StartRename();
}

void SColorThemesViewer::DuplicateColorTheme()
{
	// Create a copy of the existing current color theme
	TArray< TSharedPtr<FColorInfo> > NewColors;
	const TArray< TSharedPtr<FColorInfo> >& CurrentColors = CurrentlySelectedThemePtr.Pin()->GetColors();
	for (int32 ColorIndex = 0; ColorIndex < CurrentColors.Num(); ++ColorIndex)
	{
		NewColors.Add(MakeShareable(new FColorInfo(CurrentColors[ColorIndex]->Color, CurrentColors[ColorIndex]->Label)));
	}
	const FText Name = FText::Format(LOCTEXT("CopyThemeNameAppend", "{0} Copy"), FText::FromString(CurrentlySelectedThemePtr.Pin()->Name));
	TSharedPtr<FColorTheme> NewTheme = NewColorTheme( Name.ToString(), NewColors );
	SaveColorThemesToIni();

	SetCurrentColorTheme(NewTheme);
	StartRename();
}

void SColorThemesViewer::DeleteColorTheme()
{
	// Delete the current color theme
	ColorThemes.Remove(GetCurrentColorTheme());

	if (!ColorThemes.Num())
	{
	 	// Create the default if none exists
		GetDefaultColorTheme();
	}

	SetCurrentColorTheme(Recents);
	SaveColorThemesToIni();

	RefreshMenuWidget();
}

void SColorThemesViewer::LoadColorThemesFromIni()
{
	if (FPaths::FileExists(GEditorPerProjectIni))
	{
		// Load Themes
		bool bThemesRemaining = true;
		int32 ThemeID = 0;
		while (bThemesRemaining)
		{
			const FString ThemeName = GConfig->GetStr(TEXT("ColorThemes"), *FString::Printf(TEXT("Theme%i"), ThemeID), GEditorPerProjectIni);
			if (!ThemeName.IsEmpty())
			{
				TSharedPtr<FColorTheme> ColorTheme = GetColorTheme(ThemeName);
				check( ColorTheme.IsValid() );
				bool bColorsRemaining = true;
				int32 ColorID = 0;
				while (bColorsRemaining)
				{
					const FString ColorString = GConfig->GetStr(TEXT("ColorThemes"), *FString::Printf(TEXT("Theme%iColor%i"), ThemeID, ColorID), GEditorPerProjectIni);
					if (!ColorString.IsEmpty())
					{
						// Add the color if it hasn't already
						FLinearColor Color;
						Color.InitFromString(ColorString);
						if ( ColorTheme->FindApproxColor( Color ) == INDEX_NONE )
						{
							TSharedPtr<FColorInfo> NewColor = MakeShareable(new FColorInfo(MakeShareable(new FLinearColor(Color))));
							const FString LabelString = GConfig->GetStr(TEXT("ColorThemes"), *FString::Printf(TEXT("Theme%iLabel%i"), ThemeID, ColorID), GEditorPerProjectIni);
							if (!LabelString.IsEmpty())
							{
								NewColor->Label = FText::FromString(LabelString);
							}
							ColorTheme->InsertNewColor(NewColor, 0);
						}
						++ColorID;
					}
					else
					{
						bColorsRemaining = false;
					}
				}
				++ThemeID;
			}
			else
			{
				bThemesRemaining = false;
			}
		}

		// Load Recents
		bool bColorsRemaining = true;
		int32 ColorID = 0;
		while (bColorsRemaining)
		{
			const FString ColorString = GConfig->GetStr(TEXT("RecentColors"), *FString::Printf(TEXT("Color%i"), ColorID), GEditorPerProjectIni);
			if (!ColorString.IsEmpty())
			{
				// Add the color if it hasn't already
				FLinearColor Color;
				Color.InitFromString(ColorString);
				if (Recents->FindApproxColor(Color) == INDEX_NONE)
				{
					TSharedPtr<FColorInfo> NewColor = MakeShareable(new FColorInfo(MakeShareable(new FLinearColor(Color))));
					Recents->InsertNewColor(NewColor, 0);
				}
				++ColorID;
			}
			else
			{
				bColorsRemaining = false;
			}
		}
	}
	
	if (!ColorThemes.Num())
	{
	 	// Create the default if none exists
		GetDefaultColorTheme();
	}

	// Load the currently selected theme
	if (FPaths::FileExists(GEditorPerProjectIni))
	{
		const FString ThemeName = GConfig->GetStr(TEXT("ColorPickerUI"), TEXT("CurrentTheme"), GEditorPerProjectIni);

		if (TSharedPtr<FColorTheme>* CurrentTheme = Algo::FindBy(ColorThemes, ThemeName, &FColorTheme::Name))
		{
			CurrentlySelectedThemePtr = *CurrentTheme;
		}
	}
}

void SColorThemesViewer::SaveColorThemesToIni()
{
	if (FPaths::FileExists(GEditorPerProjectIni))
	{
		// Save Color Themes
		GConfig->EmptySection(TEXT("ColorThemes"), GEditorPerProjectIni);

		for (int32 ThemeIndex = 0; ThemeIndex < ColorThemes.Num(); ++ThemeIndex)
		{
			const TSharedPtr<FColorTheme>& Theme = ColorThemes[ThemeIndex];
			GConfig->SetString(TEXT("ColorThemes"), *FString::Printf(TEXT("Theme%i"), ThemeIndex), *Theme->Name, GEditorPerProjectIni);

			const TArray< TSharedPtr<FColorInfo> >& Colors = Theme->GetColors();
			for (int32 ColorIndex = 0; ColorIndex < Colors.Num(); ++ColorIndex)
			{
				const TSharedPtr<FLinearColor>& Color = Colors[ColorIndex]->Color;
				const FText& Label = Colors[ColorIndex]->Label;
				GConfig->SetString(TEXT("ColorThemes"), *FString::Printf(TEXT("Theme%iColor%i"), ThemeIndex, ColorIndex), *Color->ToString(), GEditorPerProjectIni);
				GConfig->SetString(TEXT("ColorThemes"), *FString::Printf(TEXT("Theme%iLabel%i"), ThemeIndex, ColorIndex), *Label.ToString(), GEditorPerProjectIni);
			}
		}

		// Save Recents
		GConfig->EmptySection(TEXT("RecentColors"), GEditorPerProjectIni);

		const TArray< TSharedPtr<FColorInfo> >& Colors = Recents->GetColors();
		for (int32 ColorIndex = 0; ColorIndex < Colors.Num(); ++ColorIndex)
		{
			const TSharedPtr<FLinearColor>& Color = Colors[ColorIndex]->Color;
			const FText& Label = Colors[ColorIndex]->Label;
			GConfig->SetString(TEXT("RecentColors"), *FString::Printf(TEXT("Color%i"), ColorIndex), *Color->ToString(), GEditorPerProjectIni);
		}
	}
}

#undef LOCTEXT_NAMESPACE