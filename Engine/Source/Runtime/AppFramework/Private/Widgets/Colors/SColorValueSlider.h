// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Bad header, does not compile

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "InputCoreTypes.h"
#include "Input/Reply.h"
#include "Rendering/DrawElements.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SLeafWidget.h"
#include "Framework/SlateDelegates.h"

class FPaintArgs;
class FSlateRect;
class FWidgetStyle;
struct FGeometry;
struct FPointerEvent;
struct FSlateBrush;

/**
 * The value slider is a simple control like the color wheel for selecting value.
 */
class SColorValueSlider
	: public SLeafWidget
{
public:

	SLATE_BEGIN_ARGS( SColorValueSlider )
		: _SelectedColor()
		, _OnValueChanged()
		, _OnMouseCaptureBegin()
		, _OnMouseCaptureEnd()
	{ }
		
		/** The current color selected by the user */
		SLATE_ATTRIBUTE(FLinearColor, SelectedColor)
		
		/** Invoked when a new value is selected on the color wheel */
		SLATE_EVENT(FOnLinearColorValueChanged, OnValueChanged)
		
		/** Invoked when the mouse is pressed and sliding begins */
		SLATE_EVENT(FSimpleDelegate, OnMouseCaptureBegin)

		/** Invoked when the mouse is released and sliding ends */
		SLATE_EVENT(FSimpleDelegate, OnMouseCaptureEnd)

	SLATE_END_ARGS()
	
	void Construct( const FArguments& InArgs )
	{
		SelectorImage = FCoreStyle::Get().GetBrush("ColorPicker.Selector");

		OnValueChanged = InArgs._OnValueChanged;
		OnMouseCaptureBegin = InArgs._OnMouseCaptureBegin;
		OnMouseCaptureEnd = InArgs._OnMouseCaptureEnd;
		SelectedColor = InArgs._SelectedColor;
	}

	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
	{
		const bool bIsEnabled = ShouldBeEnabled(bParentEnabled);
		const ESlateDrawEffect DrawEffects = bIsEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	
		FLinearColor FullValueColor = SelectedColor.Get();
		FullValueColor.B = FullValueColor.A = 1.f;
		FLinearColor StopColor = FullValueColor.HSVToLinearRGB();

		TArray<FSlateGradientStop> GradientStops;
		GradientStops.Add(FSlateGradientStop(FVector2D::ZeroVector, FLinearColor(0.0f, 0.0f, 0.0f, 1.0f)));
		GradientStops.Add(FSlateGradientStop(AllottedGeometry.Size, StopColor));

		FSlateDrawElement::MakeGradient(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			GradientStops,
			Orient_Vertical,
			MyCullingRect,
			DrawEffects
		);
	
		float Value = SelectedColor.Get().B;
		FVector2D RelativeSelectedPosition = FVector2D(Value, 0.5f);

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + 1,
			AllottedGeometry.ToPaintGeometry( RelativeSelectedPosition*AllottedGeometry.Size - SelectorImage->ImageSize*0.5, SelectorImage->ImageSize ),
			SelectorImage,
			MyCullingRect,
			DrawEffects,
			InWidgetStyle.GetColorAndOpacityTint() * SelectorImage->GetTint( InWidgetStyle ) );

		return LayerId + 1;
	}
	
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
	{
		if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
		{
			OnMouseCaptureBegin.ExecuteIfBound();

			return FReply::Handled().CaptureMouse( SharedThis(this) );
		}
		else
		{
			return FReply::Unhandled();
		}
	}
	
	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
	{
		if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && HasMouseCapture() )
		{
			OnMouseCaptureEnd.ExecuteIfBound();

			return FReply::Handled().ReleaseMouseCapture();
		}
		else
		{
			return FReply::Unhandled();
		}
	}

	virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
	{
		if (!HasMouseCapture())
		{
			return FReply::Unhandled();
		}

		FVector2D LocalMouseCoordinate = MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() );
		FVector2D Location = LocalMouseCoordinate / (MyGeometry.Size);
		float Value = FMath::Clamp(Location.X, 0.f, 1.f);

		FLinearColor NewColor = SelectedColor.Get();
		NewColor.B = Value;

		OnValueChanged.ExecuteIfBound(NewColor);

		return FReply::Handled();
	}
	
	virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
	{
		return FReply::Handled();
	}

	virtual FVector2D ComputeDesiredSize(float) const
	{
		return SelectorImage->ImageSize * 2.f;
	}
	
protected:

	/** The color selector image to show */
	const FSlateBrush* SelectorImage;
	
	/** The current color selected by the user */
	TAttribute< FLinearColor > SelectedColor;

	/** Invoked when a new value is selected on the color wheel */
	FOnLinearColorValueChanged OnValueChanged;

	/** Invoked when the mouse is pressed */
	FSimpleDelegate OnMouseCaptureBegin;

	/** Invoked when the mouse is let up */
	FSimpleDelegate OnMouseCaptureEnd;
};
