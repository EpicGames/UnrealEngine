// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Testing/SPerfSuite.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Application/SlateWindowHelper.h"
#include "Framework/Docking/TabManager.h"

#if !UE_BUILD_SHIPPING

#include "Widgets/Docking/SDockTab.h"
#include "Framework/Testing/STableViewTesting.h"
#include "ISlateReflectorModule.h"

void SummonPerfTestSuite()
{
	// Need to load this module so we have the widget reflector tab available
	FModuleManager::LoadModuleChecked<ISlateReflectorModule>("SlateReflector");

	auto SpawnTableViewTesting = [](const FSpawnTabArgs&) -> TSharedRef<SDockTab>
	{
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				MakeTableViewTesting()
			];
	};

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner("TableViewTesting", FOnSpawnTab::CreateLambda(SpawnTableViewTesting));
	
	TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout( "PerfTestSuite_Layout" )
	->AddArea
	(
		FTabManager::NewArea(1920,1200)
		->Split
		(
			FTabManager::NewStack()
			->AddTab("TableViewTesting", ETabState::OpenedTab)
		)
	)
	->AddArea
	(
		FTabManager::NewArea(640,800)
		->Split
		(
			FTabManager::NewStack()->AddTab("WidgetReflector", ETabState::OpenedTab)
		)
	);

	FGlobalTabmanager::Get()->RestoreFrom(Layout, TSharedPtr<SWindow>());
}

#endif // #if !UE_BUILD_SHIPPING
