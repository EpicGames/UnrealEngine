// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Testing/SStarshipSuite.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Application/SlateWindowHelper.h"
#include "Framework/Docking/TabManager.h"

#if !UE_BUILD_SHIPPING

#include "Widgets/Docking/SDockTab.h"

#include "Framework/Testing/SStarshipGallery.h"

#include "Styling/StarshipCoreStyle.h"
#include "Styling/AppStyle.h"

#include "ISlateReflectorModule.h"

#include "Brushes/SlateDynamicImageBrush.h"

const FLazyName StarshipGalleryTabName("STARSHIP GALLERY");

void RegisterStarshipGalleryTabSpawner()
{
	auto SpawnGallery = [](const FSpawnTabArgs&) -> TSharedRef<SDockTab>
	{
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				MakeStarshipGallery()
			];
	};
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(StarshipGalleryTabName, FOnSpawnTab::CreateLambda(SpawnGallery));
}

void UnregisterStarshipGalleryTabSpawner()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(StarshipGalleryTabName);
}

void RestoreStarshipSuite()
{

	// Need to load this module so we have the widget reflector tab available
	FModuleManager::LoadModuleChecked<ISlateReflectorModule>("SlateReflector");

	RegisterStarshipGalleryTabSpawner();
	
	TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout( "StarshipSuite_Layout" )
	->AddArea
	(
		FTabManager::NewArea(1230, 900)
		->Split
		(
			FTabManager::NewStack()
			->AddTab(StarshipGalleryTabName, ETabState::OpenedTab)
			->AddTab("WidgetReflector", ETabState::OpenedTab)
			->SetForegroundTab(FName(StarshipGalleryTabName))
		)
	);

	FGlobalTabmanager::Get()->RestoreFrom(Layout, TSharedPtr<SWindow>());
}

#endif // #if !UE_BUILD_SHIPPING
