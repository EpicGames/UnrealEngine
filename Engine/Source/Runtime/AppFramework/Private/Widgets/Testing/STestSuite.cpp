// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Testing/STestSuite.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "InputCoreTypes.h"
#include "SlateGlobals.h"
#include "Layout/Visibility.h"
#include "Types/ISlateMetaData.h"
#include "Layout/Margin.h"
#include "Math/TransformCalculus2D.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Animation/CurveHandle.h"
#include "Animation/CurveSequence.h"
#include "Styling/SlateColor.h"
#include "Textures/SlateShaderResource.h"
#include "Fonts/SlateFontInfo.h"
#include "Fonts/FontMeasure.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyle.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/DrawElementPayloads.h"
#include "Rendering/DrawElements.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SWindow.h"
#include "Widgets/SUserWidget.h"
#include "Layout/WidgetPath.h"
#include "SlateOptMacros.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/ISlateRun.h"
#include "Framework/Text/SlateTextRun.h"
#include "Framework/Text/SlateHyperlinkRun.h"
#include "Framework/Text/SlateWidgetRun.h"
#include "Widgets/Layout/SFxWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/Text/ITextDecorator.h"
#include "Framework/Text/TextDecorators.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Framework/Docking/WorkspaceItem.h"
#include "Framework/Docking/TabManager.h"
#include "HAL/PlatformApplicationMisc.h"

#if !UE_BUILD_SHIPPING

#include "ISlateReflectorModule.h"
#include "Framework/Testing/STableViewTesting.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SVolumeControl.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SDPIScaler.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Testing/SLayoutExample.h"
#include "Framework/Testing/SWidgetGallery.h"
#include "Framework/Testing/TestStyle.h"
#include "Framework/Text/RichTextLayoutMarshaller.h"
#include "Framework/Text/SyntaxHighlighterTextLayoutMarshaller.h"
#include "Math/TransformCalculus3D.h"
#include "Widgets/Layout/SResponsiveGridPanel.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Notifications/INotificationWidget.h"
#include "Widgets/SInvalidationPanel.h"


#define LOCTEXT_NAMESPACE "STestSuite"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

namespace
{
	/**
	 * A set of syntax tests for transform calculus to ensure that the basic primitives are supported properly.
	 * Does NOT test for correctness, purely syntax!
	 */
	bool TestTransformCalculus()
	{
		float uniScale = 5.8f;
		FVector trans(5, 6, 7);
		FQuat quat(FVector(1, 2, 3).GetSafeNormal(), 33.5f);
		FRotator rot(7, 8, 9);
		FMatrix mat = FRotationMatrix::Make(rot);

		// Identity casts
		const float& uniScale2 = TransformCast<float>(uniScale);
		const FVector& trans2 = TransformCast<FVector>(trans);
		const FQuat& quat2 = TransformCast<FQuat>(quat);
		const FRotator& rot2 = TransformCast<FRotator>(rot);
		const FMatrix& mat2 = TransformCast<FMatrix>(mat);

		// rotation casts
		auto quat3 = TransformCast<FQuat>(rot);
		auto rot3 = TransformCast<FRotator>(quat);
		// PVS-Studio complains about double initialization, but that's something we're testing
		// so we disable the warning:
		rot3 = TransformCast<FRotator>(mat); //-V519

		// higher level transform casts
		auto mat3 = TransformCast<FMatrix>(uniScale);
		mat3 = TransformCast<FMatrix>(trans);
		mat3 = TransformCast<FMatrix>(quat);
		mat3 = TransformCast<FMatrix>(rot);

		// basic concatenation
		auto uniScale4 = Concatenate(uniScale, uniScale2);
		auto trans4 = Concatenate(trans, trans2);
		auto quat4 = Concatenate(quat, quat2);
		auto rot4 = Concatenate(rot, rot2);
		auto mat4 = Concatenate(mat, mat2);

		// matrix conversion
		FMatrix mat5;
		FQuat quat5;

		mat5 = Concatenate(uniScale4, trans4);
		mat5 = Concatenate(trans4, uniScale4);
		mat5 = Concatenate(uniScale4, quat4);
		mat5 = Concatenate(quat4, uniScale4);
		mat5 = Concatenate(uniScale4, rot4);
		mat5 = Concatenate(rot4, uniScale4);
		mat5 = Concatenate(uniScale4, mat4);
		mat5 = Concatenate(mat4, uniScale4);

		mat5 = Concatenate(trans4, quat4);
		mat5 = Concatenate(quat4, trans4);
		mat5 = Concatenate(trans4, rot4);
		mat5 = Concatenate(rot4, trans4);
		mat5 = Concatenate(trans4, mat4);
		mat5 = Concatenate(mat4, trans4);

		quat5 = Concatenate(quat4, rot4);
		quat5 = Concatenate(rot4, quat4); //-V519
		mat5 = Concatenate(quat4, mat4);
		mat5 = Concatenate(mat4, quat4);

		mat5 = Concatenate(rot4, mat4);
		mat5 = Concatenate(mat4, rot4);

		mat5 = Concatenate(Inverse(trans), rot, trans);

		mat5 = Concatenate(Inverse(trans), rot, uniScale, trans);

		FScale2D scale2D(1.4f, 3.2f);
		FShear2D shear2D(-3.5f, -4.6f);
		FVector2D trans2D(7, 8);
		FQuat2D rot2D(1.4f);
		FMatrix2x2 mat2D(5, 6, 7, 8);
		FSlateLayoutTransform transformLayout(uniScale, trans2D);
		FSlateRenderTransform transform2D = ::Concatenate(FScale2D(2, 4), FShear2D(0.5, 2), FQuat2D(FMath::DegreesToRadians(45.0f)), FVector2D(5, 7));

		// Identity casts
		const FScale2D& scale2D2 = TransformCast<FScale2D>(scale2D);
		const FShear2D& shear2D2 = TransformCast<FShear2D>(shear2D);
		const FVector2D& trans2D2 = TransformCast<FVector2D>(trans2D);
		const FQuat2D& rot2D2 = TransformCast<FQuat2D>(rot2D);
		const FMatrix2x2& mat2D2 = TransformCast<FMatrix2x2>(mat2D);
		const FSlateLayoutTransform& transformLayout2 = TransformCast<FSlateLayoutTransform>(transformLayout);
		const FSlateRenderTransform& transform2D2 = TransformCast<FSlateRenderTransform>(transform2D);

		auto scale2D3 = Concatenate(scale2D, scale2D);
		auto shear2D3 = Concatenate(shear2D, shear2D);
		auto trans2D3 = Concatenate(trans2D, trans2D);
		auto rot2D3 = Concatenate(rot2D, rot2D);
		auto mat2D3 = Concatenate(mat2D, mat2D);
		auto transformLayout3 = Concatenate(transformLayout, transformLayout);
		auto transform2D3 = Concatenate(transform2D, transform2D);

		// higher level transform casts
		transformLayout3 = TransformCast<FSlateLayoutTransform>(uniScale);
		transformLayout3 = TransformCast<FSlateLayoutTransform>(trans2D);

		transform2D3 = TransformCast<FSlateRenderTransform>(uniScale);
		transform2D3 = TransformCast<FSlateRenderTransform>(scale2D);
		transform2D3 = TransformCast<FSlateRenderTransform>(shear2D);
		transform2D3 = TransformCast<FSlateRenderTransform>(trans2D);
		transform2D3 = TransformCast<FSlateRenderTransform>(rot2D);
		transform2D3 = TransformCast<FSlateRenderTransform>(mat2D);
		transform2D3 = TransformCast<FSlateRenderTransform>(transformLayout);

		// basic concatenation
		scale2D3 = Concatenate(uniScale, scale2D);
		scale2D3 = Concatenate(scale2D, uniScale);

		auto transformLayout4 = Concatenate(transformLayout, transformLayout);
		transformLayout4 = Concatenate(uniScale, trans2D);
		transformLayout4 = Concatenate(trans2D, uniScale);

		transformLayout4 = Concatenate(uniScale, transformLayout);
		transformLayout4 = Concatenate(trans2D, transformLayout);
		transformLayout4 = Concatenate(transformLayout, uniScale);
		transformLayout4 = Concatenate(transformLayout, trans2D);

		auto transform2D4 = Concatenate(transform2D, transform2D);

		transform2D4 = Concatenate(trans2D, scale2D);
		transform2D4 = Concatenate(trans2D, shear2D);
		transform2D4 = Concatenate(trans2D, rot2D);
		transform2D4 = Concatenate(trans2D, mat2D);
		transform2D4 = Concatenate(trans2D, transform2D);
		transform2D4 = Concatenate(scale2D, transformLayout);
		transform2D4 = Concatenate(scale2D, transform2D);
		transform2D4 = Concatenate(shear2D, transformLayout);
		transform2D4 = Concatenate(shear2D, transform2D);
		transform2D4 = Concatenate(rot2D, transformLayout);
		transform2D4 = Concatenate(rot2D, transform2D);
		transform2D4 = Concatenate(mat2D, transformLayout);
		transform2D4 = Concatenate(mat2D, transform2D);
		transform2D4 = Concatenate(transformLayout, transform2D);

		transform2D4 = Concatenate(scale2D, trans2D);
		transform2D4 = Concatenate(shear2D, trans2D);
		transform2D4 = Concatenate(rot2D, trans2D);
		transform2D4 = Concatenate(mat2D, trans2D);
		transform2D4 = Concatenate(transform2D, trans2D);
		transform2D4 = Concatenate(transformLayout, scale2D);
		transform2D4 = Concatenate(transform2D, scale2D);
		transform2D4 = Concatenate(transformLayout, shear2D);
		transform2D4 = Concatenate(transform2D, shear2D);
		transform2D4 = Concatenate(transformLayout, rot2D);
		transform2D4 = Concatenate(transform2D, rot2D);
		transform2D4 = Concatenate(transformLayout, mat2D);
		transform2D4 = Concatenate(transform2D, mat2D);
		transform2D4 = Concatenate(transform2D, transformLayout);

		auto mat2D4 = Concatenate(scale2D, shear2D);
		mat2D4 = Concatenate(scale2D, rot2D);
		mat2D4 = Concatenate(scale2D, mat2D);

		mat2D4 = Concatenate(shear2D, scale2D);
		mat2D4 = Concatenate(rot2D, scale2D);
		mat2D4 = Concatenate(mat2D, scale2D);

		mat2D4 = Concatenate(shear2D, rot2D);
		mat2D4 = Concatenate(shear2D, mat2D);

		mat2D4 = Concatenate(rot2D, shear2D);
		mat2D4 = Concatenate(mat2D, shear2D);

		mat2D4 = Concatenate(rot2D, mat2D);

		mat2D4 = Concatenate(mat2D, rot2D);
		{
			auto matRot = Concatenate(FRotationMatrix::Make(FRotator(0.0f, 17.5f, 0.0f)), FRotationMatrix::Make(FRotator(12.4f, 5.7f, 29.0f)));
			auto quatRot = Concatenate(FQuat(FRotator(0.0f, 17.5f, 0.0f)), FQuat(FRotator(12.4f, 5.7f, 29.0f)));

			FVector vec(3.0f, 4.0f, 5.0f);
			auto matRotRes = TransformPoint(matRot, vec);
			auto quatRotRes = TransformPoint(quatRot, vec);
		}

		return true;
	}
	bool bTestTransformCalculus = TestTransformCalculus();
}

struct FOnPaintHandlerParams
{
	const FGeometry& Geometry;
	const FSlateRect& ClippingRect;
	FSlateWindowElementList& OutDrawElements;
	const int32 Layer;
	const bool bEnabled;

	FOnPaintHandlerParams( const FGeometry& InGeometry, const FSlateRect& InClippingRect, FSlateWindowElementList& InOutDrawElements, int32 InLayer, bool bInEnabled )
		: Geometry( InGeometry )
		, ClippingRect( InClippingRect )
		, OutDrawElements( InOutDrawElements )
		, Layer( InLayer )
		, bEnabled( bInEnabled )
	{
	}

};

/** Delegate type for allowing custom OnPaint handlers */
DECLARE_DELEGATE_RetVal_OneParam( 
	int32,
	FOnPaintHandler,
	const FOnPaintHandlerParams& );

/** Widget with a handler for OnPaint; convenient for testing various DrawPrimitives. */
class SCustomPaintWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SCustomPaintWidget )
		: _OnPaintHandler()
		{}

		SLATE_EVENT( FOnPaintHandler, OnPaintHandler )
	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs)
	{
		OnPaintHandler = InArgs._OnPaintHandler;
	}

	virtual FVector2D ComputeDesiredSize(float) const override
	{
		return FVector2D(128, 128);
	}

	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override
	{
		if( OnPaintHandler.IsBound() )
		{
			FOnPaintHandlerParams Params( AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, bParentEnabled && IsEnabled() ); 
			OnPaintHandler.Execute( Params );
		}
		else
		{
			FSlateDrawElement::MakeDebugQuad(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry()
			);
		}

		return SCompoundWidget::OnPaint( Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled && IsEnabled() );
	}

private:
	FOnPaintHandler OnPaintHandler;
};

class SDynamicBrushTest : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SDynamicBrushTest ){}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs )
	{
		ChildSlot
		[
			SNew( SVerticalBox )
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.HAlign( HAlign_Left )
			.VAlign( VAlign_Top )
			[
				SNew( SBorder )
				[
					SNew( SBox )
					.WidthOverride( 128 )
					.HeightOverride( 128 )
					[
						SNew( SImage )
						.Image( this, &SDynamicBrushTest::GetImage )
					]
				]
			]
			+ SVerticalBox::Slot()
			.FillHeight(.2f)
			.HAlign( HAlign_Left )
			[
				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew( SEditableTextBox )
					.Text( this, &SDynamicBrushTest::GetFilenameText )
					.HintText( LOCTEXT("DynamicBrushTestLabel", "Type in full path to an image (png)") )
					.OnTextCommitted( this, &SDynamicBrushTest::LoadImage )

				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding( 2.0f )
				.VAlign( VAlign_Center )
				[
					SNew( SButton )
					.ContentPadding( 1.0f )
					.Text( LOCTEXT("ResetLabel", "Reset") )
					.OnClicked( this, &SDynamicBrushTest::Reset )
				]
			]
		];
	}

	~SDynamicBrushTest()
	{
		Reset();
	}
private:
	const FSlateBrush* GetImage() const
	{
		return DynamicBrush.IsValid() ? DynamicBrush.Get() : FCoreStyle::Get().GetBrush("Checkerboard");
	}


	void LoadImage( const FText& Text, ETextCommit::Type CommitType )
	{
		FilenameText = Text;
		FString Filename = Text.ToString();
		// Note Slate will append the extension automatically so remove the extension
		FName BrushName( *FPaths::GetBaseFilename( Filename, false ) );

		DynamicBrush = MakeShareable( new FSlateDynamicImageBrush( BrushName, FVector2D(128,128) ) );
	}

	FReply Reset()
	{
		FilenameText = FText::GetEmpty();
		DynamicBrush.Reset();
		return FReply::Handled();
	}

	FText GetFilenameText() const
	{
		return FilenameText;
	}
private:
	TSharedPtr<FSlateDynamicImageBrush> DynamicBrush;
	FText FilenameText;
};


//extern float SlateSplineNormalThreshold;
class SSplineWithHandles : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSplineWithHandles)
	: _P0(FVector2f(0.f,32.f))
	, _P1(FVector2f(100.f, 32.f))
	, _P2(FVector2f(0.f,132.f))
	, _P3(FVector2f(100.f,132.f))
	, _SplineThickness(1.0f)
	{}
		SLATE_ARGUMENT(FVector2f, P0)
		SLATE_ARGUMENT(FVector2f, P1)
		SLATE_ARGUMENT(FVector2f, P2)
		SLATE_ARGUMENT(FVector2f, P3)
		SLATE_ATTRIBUTE(float, SplineThickness)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		SetPoints(InArgs._P0, InArgs._P1, InArgs._P2, InArgs._P3);
		SplineThickness = InArgs._SplineThickness;
	}

	void SetPoints(FVector2f P0, FVector2f P1, FVector2f P2, FVector2f P3)
	{
		BezierPoints[0] = P0;
		BezierPoints[1] = P1;
		BezierPoints[2] = P2;
		BezierPoints[3] = P3;
	}

	void SetGradient(const TArray<FSlateGradientStop>& InGradient)
	{
		GradientStops = InGradient;
	}

private:
	const FVector2f BezierPointRadius = FVector2f(8.f,8.f);

	TSharedPtr<SSpinBox<float>> ThicknessSpinBox;
	
	//~ SWidget
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		const bool bEnabled = ShouldBeEnabled(bParentEnabled);
		
		// Draw control points.
		for (int32 i = 0; i < 4; ++i)
		{
			const FVector2f RadiusToUse = (i == 0 || i == 3) ? BezierPointRadius : 0.5f*BezierPointRadius;
			const FLinearColor ColorToUse = (i == 0 || i == 1) ? FLinearColor::Green : FLinearColor::Blue;

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(FVector2D(2.f* RadiusToUse), FSlateLayoutTransform(BezierPoints[i]- RadiusToUse)),
				&WhiteBox,
				bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
				ColorToUse * FLinearColor(1.f, 1.f, 1.f, 0.25f)
			);

		}

		++LayerId;


		// Draw the spline
#ifdef ENABLE_SPLINE_WITH_GRADIENT_TESTING
		if (GradientStops.Num() > 0)
		{
			// The legacy gradient function only supported draw-space inputs and only in hermite coordinates.
			// Accommodate its inputs for testing purposes.
			const FSlateRenderTransform ThisRenderTransform = AllottedGeometry.GetAccumulatedRenderTransform();
			const FVector2D Hermite_P0 = ThisRenderTransform.TransformPoint(BezierPoints[0]);
			const FVector2D Hermite_M0 = 3.0f*(ThisRenderTransform.TransformPoint(BezierPoints[1]) - Hermite_P0);
			const FVector2D Hermite_P1 = ThisRenderTransform.TransformPoint(BezierPoints[3]);
			const FVector2D Hermite_M1 = 3.0f*(Hermite_P1 - ThisRenderTransform.TransformPoint(BezierPoints[2]));

			FSlateDrawElement::MakeDrawSpaceGradientSpline(
				OutDrawElements,
				LayerId,
				Hermite_P0, Hermite_M0,
				Hermite_P1, Hermite_M1,
				GradientStops,
				SplineThickness.Get() * AllottedGeometry.Scale,
				bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect
			);

			TArray<FSlateGradientStop> GradientStopsForReference= GradientStops;
			for (FSlateGradientStop& SomeStop : GradientStopsForReference)
			{
				SomeStop.Position.X *= AllottedGeometry.Size.X;
			}
			FSlateDrawElement::MakeGradient(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(FVector2D(AllottedGeometry.Size.X, 16), FSlateLayoutTransform()),
				GradientStopsForReference,
				Orient_Vertical
			);
		}
		else
#endif//ENABLE_SPLINE_WITH_GRADIENT_TESTING
		{
			FSlateDrawElement::MakeCubicBezierSpline(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(),
				BezierPoints[0], BezierPoints[1], BezierPoints[2], BezierPoints[3],
				SplineThickness.Get() * AllottedGeometry.Scale,
				bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
				FColor::White
			);
		}

		return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.GetEffectingButton()==EKeys::LeftMouseButton)
		{
			const FVector2f LocalCursorPos = UE::Slate::CastToVector2f(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()));
			PointBeingDragged = PointIndexFromCursorPos(BezierPoints, LocalCursorPos, 2*BezierPointRadius.X*2*BezierPointRadius.X);

			if (PointBeingDragged != INDEX_NONE)
			{
				return FReply::Handled().CaptureMouse(SharedThis(this));
			}
		}

		return FReply::Unhandled();
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (PointBeingDragged != INDEX_NONE && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			PointBeingDragged = INDEX_NONE;
			return FReply::Handled().ReleaseMouseCapture();
		}
		else
		{
			return FReply::Unhandled();
		}
	}

	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (PointBeingDragged != INDEX_NONE)
		{
			const FVector2f LocalCursorPos = UE::Slate::CastToVector2f(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()));
			BezierPoints[PointBeingDragged] = LocalCursorPos;
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}
	//~ SWidget


	static int32 PointIndexFromCursorPos(const FVector2f* Points, const FVector2f LocalCursorPos, const float RadiusSquared)
	{
		for (int i = 0; i < 4; ++i)
		{			
			const float SizeSquared = (Points[i] - LocalCursorPos).SizeSquared();
			if (SizeSquared < RadiusSquared)
			{
				return i;
			}
		}
		return INDEX_NONE;
	}

	FSlateColorBrush WhiteBox = FSlateColorBrush(FColor::White);
	FVector2f BezierPoints[4];
	int32 PointBeingDragged = INDEX_NONE;
	TAttribute<float> SplineThickness;
	TArray<FSlateGradientStop> GradientStops;
};

class SSplineTest : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSplineTest)
		: _P0(FVector2f(0.f, 32.f))
		, _P1(FVector2f(100.f, 32.f))
		, _P2(FVector2f(0.f, 132.f))
		, _P3(FVector2f(100.f, 132.f))
	{}
		SLATE_ARGUMENT(FVector2f, P0)
		SLATE_ARGUMENT(FVector2f, P1)
		SLATE_ARGUMENT(FVector2f, P2)
		SLATE_ARGUMENT(FVector2f, P3)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		this->ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SNew(SSpinBox<float>)
					.MinValue(0.1f)
					.MaxValue(20.0f)
					.Value(this, &SSplineTest::GetSplineThickness)
					.OnValueChanged(this, &SSplineTest::OnSplineThicknessChanged)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SNew(SSpinBox<int32>)
					.MinValue(0)
					.MaxValue(16)
					.OnValueChanged(this, &SSplineTest::OnNumGradientStopsChanged)
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SButton)
					.Text(LOCTEXT("HorizontalHermite", "Horizontal"))
					.OnClicked_Lambda([this]()
					{
						SplineWidget->SetPoints(FVector2f(0.f, 64.f), FVector2f(66.f, 64.f), FVector2f(133.f, 64.f), FVector2f(200.f, 64.f));
						return FReply::Handled();
					})
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SButton)
					.Text(LOCTEXT("VerticalHermite", "Vertical"))
					.OnClicked_Lambda([this]()
					{
						SplineWidget->SetPoints(FVector2f(64.f,0.f), FVector2f(64.f, 66.f), FVector2f(64.f, 133.f), FVector2f(64.f, 200.f));
						return FReply::Handled();
					})
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SButton)
					.Text(LOCTEXT("HorizontalBendy", "Bendy"))
					.OnClicked_Lambda([this]()
					{
						SplineWidget->SetPoints(FVector2f(64.f, 64.f), FVector2f(128.f, 64.f), FVector2f(64.f, 128.f), FVector2f(128.f, 128.f));
						return FReply::Handled();
					})
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SButton)
					.Text(LOCTEXT("HorizontalDoubleBendy", "DoubleBendy"))
					.OnClicked_Lambda([this]()
					{
						SplineWidget->SetPoints(FVector2f(128.f, 128.f), FVector2f(128.f -96.f, 128.f), FVector2f(192.f +96.f, 192.f), FVector2f(192.f, 192.f));
						return FReply::Handled();
					})
				]
				
			]			
			+ SVerticalBox::Slot()
			.FillHeight(1)
			[
				SAssignNew(SplineWidget, SSplineWithHandles)
				.P0(InArgs._P0)
				.P1(InArgs._P1)
				.P2(InArgs._P2)
				.P3(InArgs._P3)
				.SplineThickness(this, &SSplineTest::GetSplineThickness)
			]
		];
	}

private:
	float GetSplineThickness() const { return SplineThickness; }
	void OnSplineThicknessChanged(float InNewThickness) { SplineThickness = InNewThickness; }
	void OnNumGradientStopsChanged(int32 NewNumGradientStops)
	{
		const auto ColorOne = FLinearColor::MakeRandomColor();
		const auto ColorTwo = FLinearColor::MakeRandomColor();

		TArray<FSlateGradientStop> GradientStops;
		GradientStops.Reserve(NewNumGradientStops);
		for (int i = 0; i < NewNumGradientStops; ++i)
		{
			// Note that the position is ignored by the spline
			GradientStops.Add( FSlateGradientStop(FVector2f(static_cast<float>(i)/FMath::Max(1,NewNumGradientStops-1), 0.0f), i%2==0 ? ColorOne : ColorTwo) );
		}
		SplineWidget->SetGradient(GradientStops);
	}
	float SplineThickness = 5.0f;
	TSharedPtr<SSplineWithHandles> SplineWidget;

};


/** Test the draw elements . */
class SElementTesting : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SElementTesting ){}
	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 *
	 * @param InArgs   Declartion from which to construct the widget
	 */
	void Construct(const FArguments& InArgs)
	{
		FontScale = 1.0f;

		// Arrange a bunch of DrawElement tester widgets in a vertical stack.
		// Use custom OnPaint handlers.
		this->ChildSlot
		[
			SNew( SVerticalBox )
			+ SVerticalBox::Slot()
			.AutoHeight() 
			.HAlign( HAlign_Right )
			[
				SNew( SButton )
				.Text( LOCTEXT("DisableButton", "Disable") )
				.OnClicked( this, &SElementTesting::OnDisableClicked )
			]
			+SVerticalBox::Slot()
			.FillHeight(1)
			[
				SAssignNew( VerticalBox, SVerticalBox )
				+ SVerticalBox::Slot()
				.FillHeight(1)
				[
					SNew( SCustomPaintWidget )
					.OnPaintHandler( this, &SElementTesting::TestBoxElement ) 
				]
				+ SVerticalBox::Slot()
				.FillHeight(1)
				[
					SNew( SVerticalBox )
					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Left)
					.Padding( 4.0f )
					[
						SNew( SSpinBox<float> )
						.Delta( .1f )
						.Value( this, &SElementTesting::GetFontScale )
						.OnValueChanged( this, &SElementTesting::OnScaleValueChanged )
					]
					+ SVerticalBox::Slot()
					.FillHeight( 1.0f )
					[
						SNew( SCustomPaintWidget )
						.OnPaintHandler( this, &SElementTesting::TestTextElement ) 
					]
				]
				+ SVerticalBox::Slot()
				.FillHeight(1)
				[
					SNew( SCustomPaintWidget )
					.OnPaintHandler( this, &SElementTesting::TestGradientElement ) 
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew( SCustomPaintWidget )
					.OnPaintHandler( this, &SElementTesting::TestSplineElement ) 
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew( SCustomPaintWidget )
					.OnPaintHandler( this, &SElementTesting::TestDottedLineElement ) 
				]
				+ SVerticalBox::Slot()
				.FillHeight(3)
				[
					SNew( SCustomPaintWidget )
					.OnPaintHandler( this, &SElementTesting::TestRotation ) 
				]
				+ SVerticalBox::Slot()
				.FillHeight(3)
				[
					SNew(SCustomPaintWidget)
					.OnPaintHandler(this, &SElementTesting::TestCustomVerts)
				]
				+ SVerticalBox::Slot()
				.FillHeight(3)
				[
					SNew( SDynamicBrushTest )
				]
			]
			
		];
	}

	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override
	{
		CenterRotation += InDeltaTime*.3;
		if( CenterRotation > 2*PI)
		{
			CenterRotation-= 2*PI;
		}

		OuterRotation += (InDeltaTime *1.5);
		if( OuterRotation > 2*PI)
		{
			OuterRotation-= 2*PI;
		}
	}

	SElementTesting()
	{
		CenterRotation = 0;
		OuterRotation = 0;
	}

private:
	TSharedPtr<SVerticalBox> VerticalBox;
	float FontScale;
	float CenterRotation;
	float OuterRotation;

	FReply OnDisableClicked()
	{
		VerticalBox->SetEnabled( !VerticalBox->IsEnabled() );
		return FReply::Handled();
	}

	void OnScaleValueChanged( float NewScale )
	{
		FontScale = NewScale;
	}

	float GetFontScale() const
	{
		return FontScale;
	}

	int32 TestBoxElement( const FOnPaintHandlerParams& InParams )
	{
		const FSlateBrush* StyleInfo = FTestStyle::Get().GetDefaultBrush();

		FSlateDrawElement::MakeBox(
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToPaintGeometry(),
			StyleInfo,
			InParams.bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect
			);

		return InParams.Layer;
	}

	int32 TestTextElement( const FOnPaintHandlerParams& InParams )
	{
		const FText Text = LOCTEXT("TestText", "The quick brown fox jumps over the lazy dog 0123456789");
		uint32 FontSize = 14;
		FSlateFontInfo FontInfo = FCoreStyle::GetDefaultFontStyle("Regular", FontSize);
		FontInfo.OutlineSettings.OutlineColor = FLinearColor::Blue;
		FontInfo.OutlineSettings.OutlineSize = 2;

		FSlateDrawElement::MakeText(
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToPaintGeometry(InParams.Geometry.GetLocalSize(), FSlateLayoutTransform(FontScale)),
			Text.ToString(),
			FontInfo,
			InParams.bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			FColor( 255, 255, 255 )
			);

		return InParams.Layer;
	}


	int32 TestGradientElement( const FOnPaintHandlerParams& InParams )
	{
		TArray<FSlateGradientStop> GradientStops;

		GradientStops.Add( FSlateGradientStop(FVector2f(InParams.Geometry.GetLocalSize().X*.1f, 0), FColor::Yellow) );
		GradientStops.Add( FSlateGradientStop( FVector2f(InParams.Geometry.GetLocalSize().X*.25f,0), FColor::Magenta ) );
		GradientStops.Add( FSlateGradientStop( FVector2f(InParams.Geometry.GetLocalSize().X*.75f,0), FColor::Blue ) );
		GradientStops.Add( FSlateGradientStop( FVector2f(InParams.Geometry.GetLocalSize().X*0.9f,0), FColor::Green) );

		FSlateDrawElement::MakeGradient(
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToPaintGeometry(),
			GradientStops,
			Orient_Vertical,
			InParams.bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect
			);

		return InParams.Layer;
	}

	int32 TestSplineElement( const FOnPaintHandlerParams& InParams )
	{
		const FVector2f Start(10,10);
		const FVector2f StartDir(InParams.Geometry.GetLocalSize().X * 1000 / 600,0);
		const FVector2f End(InParams.Geometry.GetLocalSize().X/4, InParams.Geometry.GetLocalSize().Y-10);
		const FVector2f EndDir(InParams.Geometry.GetLocalSize().X * 1000 / 600,0);

		FSlateDrawElement::MakeSpline(
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToPaintGeometry(),
			Start, StartDir,
			End, EndDir,
			InParams.Geometry.Scale,
			InParams.bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			FColor::White
		);
	
		FVector2f LineStart =  FVector2f( InParams.Geometry.GetLocalSize().X/4, 10.0f );

		TArray<FVector2f> LinePoints;
		TArray<FLinearColor> LineColors;
		LinePoints.Add(LineStart); LineColors.Add(FLinearColor::Red);
		LinePoints.Add( LineStart + FVector2f( 100.0f, 50.0f ) );
		LinePoints.Add( LineStart + FVector2f( 200.0f, 10.0f ) );
		LinePoints.Add( LineStart + FVector2f( 300.0f, 50.0f ) );
		LinePoints.Add( LineStart + FVector2f( 400.0f, 10.0f ) );


		FSlateDrawElement::MakeLines( 
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToPaintGeometry(),
			LinePoints,
			InParams.bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			FColor::White,
			true,
			InParams.Geometry.Scale
		);

		FSlateDrawElement::MakeLines(
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToOffsetPaintGeometry(FVector2D(0.f, 10.f)),
			LinePoints,
			InParams.bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			FColor::White,
			true,
			2.f * InParams.Geometry.Scale
		);

		FSlateDrawElement::MakeLines(
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToOffsetPaintGeometry(FVector2D(0.f, 20.f)),
			LinePoints,
			InParams.bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			FColor::White,
			true,
			3.f * InParams.Geometry.Scale
		);

		FSlateDrawElement::MakeLines(
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToOffsetPaintGeometry(FVector2D(0.f, 34.f)),
			LinePoints,
			InParams.bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			FColor::White,
			true,
			4.f * InParams.Geometry.Scale
		);
			

		LineStart =  LinePoints[ LinePoints.Num() - 1 ] + FVector2f(50.f,10.f);
		LinePoints.Empty();

		static float CurTime = 0; 
		CurTime += FSlateApplication::Get().GetDeltaTime();
		CurTime = FMath::Fmod(CurTime, 2*PI);

		for( float I = 0; I < 10*PI; I+=.1f)
		{
			LinePoints.Add( LineStart + FVector2f( I*15 , 15*FMath::Sin( I + CurTime) ) );
		}

		static FColor Color = FColor::MakeRandomColor();
		FSlateDrawElement::MakeLines( 
			InParams.OutDrawElements,
			InParams.Layer+1,
			InParams.Geometry.ToPaintGeometry(),
			LinePoints,
			InParams.bEnabled ? ESlateDrawEffect::NoPixelSnapping : ESlateDrawEffect::NoPixelSnapping|ESlateDrawEffect::DisabledEffect,
			Color,
			true,
			InParams.Geometry.Scale);

		FSlateDrawElement::MakeLines(
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToOffsetPaintGeometry(FVector2D(0.f, 10.f)),
			LinePoints,
			InParams.bEnabled ? ESlateDrawEffect::NoPixelSnapping : ESlateDrawEffect::NoPixelSnapping | ESlateDrawEffect::DisabledEffect,
			Color,
			true,
			2.f * InParams.Geometry.Scale);

		FSlateDrawElement::MakeLines(
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToOffsetPaintGeometry(FVector2D(0.f, 20.f)),
			LinePoints,
			InParams.bEnabled ? ESlateDrawEffect::NoPixelSnapping : ESlateDrawEffect::NoPixelSnapping | ESlateDrawEffect::DisabledEffect,
			Color,
			true,
			3.f * InParams.Geometry.Scale);

		FSlateDrawElement::MakeLines(
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToOffsetPaintGeometry(FVector2D(0.f, 34.f)),
			LinePoints,
			InParams.bEnabled ? ESlateDrawEffect::NoPixelSnapping : ESlateDrawEffect::NoPixelSnapping | ESlateDrawEffect::DisabledEffect,
			Color,
			true,
			4.f * InParams.Geometry.Scale);

		return InParams.Layer;

	}

	int32 TestDottedLineElement( const FOnPaintHandlerParams& InParams )
	{
		const FVector2f Start(10,10);
		const FVector2f StartDir(InParams.Geometry.GetLocalSize().X * 1000 / 600,0);
		const FVector2f End(InParams.Geometry.GetLocalSize().X/4, InParams.Geometry.GetLocalSize().Y-10);
		const FVector2f EndDir(InParams.Geometry.GetLocalSize().X * 1000 / 600,0);

		static float CurTime = 0; 
		CurTime += FSlateApplication::Get().GetDeltaTime();

		float DashLengths[] = { 5.f, 20.f, 50.f };
		float DashLength = DashLengths[FMath::FloorToInt(CurTime) % 3];

		FVector2f LineStart =  FVector2f( InParams.Geometry.GetLocalSize().X/4, 10.0f );

		TArray<FVector2f> LinePoints;
		TArray<FLinearColor> LineColors;
		LinePoints.Add(LineStart); LineColors.Add(FLinearColor::Red);
		LinePoints.Add( LineStart + FVector2f( 100.0f, 50.0f ) );
		LinePoints.Add( LineStart + FVector2f( 200.0f, 10.0f ) );
		LinePoints.Add( LineStart + FVector2f( 300.0f, 50.0f ) );
		LinePoints.Add( LineStart + FVector2f( 400.0f, 10.0f ) );


		FSlateDrawElement::MakeDashedLines( 
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToPaintGeometry(),
			CopyTemp(LinePoints),
			InParams.bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			FColor::White,
			InParams.Geometry.Scale,
			DashLength,
			-CurTime * 10.f
		);

		FSlateDrawElement::MakeDashedLines(
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToOffsetPaintGeometry(FVector2D(0.f, 10.f)),
			CopyTemp(LinePoints),
			InParams.bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			FColor::White,
			5.f * InParams.Geometry.Scale,
			DashLength,
			-CurTime * 20.f
		);

		FSlateDrawElement::MakeDashedLines(
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToOffsetPaintGeometry(FVector2D(0.f, 20.f)),
			CopyTemp(LinePoints),
			InParams.bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			FColor::White,
			10.f * InParams.Geometry.Scale,
			DashLength,
			-CurTime * 30.f
		);

		FSlateDrawElement::MakeDashedLines(
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToOffsetPaintGeometry(FVector2D(0.f, 34.f)),
			CopyTemp(LinePoints),
			InParams.bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			FColor::White,
			20.f * InParams.Geometry.Scale,
			DashLength,
			-CurTime * 40.f
		);
			

		LineStart =  LinePoints[ LinePoints.Num() - 1 ] + FVector2f(50.f,10.f);
		LinePoints.Empty();

		float CurTimeModTwoPi = FMath::Fmod(CurTime, 2*PI);
		for( float I = 0; I < 10*PI; I+=.1f)
		{
			LinePoints.Add( LineStart + FVector2f( I*15 , 15*FMath::Sin( I + CurTimeModTwoPi) ) );
		}

		static FColor Color = FColor::MakeRandomColor();
		FSlateDrawElement::MakeDashedLines( 
			InParams.OutDrawElements,
			InParams.Layer+1,
			InParams.Geometry.ToPaintGeometry(),
			CopyTemp(LinePoints),
			InParams.bEnabled ? ESlateDrawEffect::NoPixelSnapping : ESlateDrawEffect::NoPixelSnapping|ESlateDrawEffect::DisabledEffect,
			Color,
			InParams.Geometry.Scale,
			DashLength,
			CurTime*15.f);

		FSlateDrawElement::MakeDashedLines(
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToOffsetPaintGeometry(FVector2D(0.f, 10.f)),
			CopyTemp(LinePoints),
			InParams.bEnabled ? ESlateDrawEffect::NoPixelSnapping : ESlateDrawEffect::NoPixelSnapping | ESlateDrawEffect::DisabledEffect,
			Color,
			2.f * InParams.Geometry.Scale,
			DashLength,
			CurTime*15.f);

		FSlateDrawElement::MakeDashedLines(
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToOffsetPaintGeometry(FVector2D(0.f, 20.f)),
			CopyTemp(LinePoints),
			InParams.bEnabled ? ESlateDrawEffect::NoPixelSnapping : ESlateDrawEffect::NoPixelSnapping | ESlateDrawEffect::DisabledEffect,
			Color,
			3.f * InParams.Geometry.Scale,
			DashLength,
			CurTime*15.f);

		FSlateDrawElement::MakeDashedLines(
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToOffsetPaintGeometry(FVector2D(0.f, 34.f)),
			CopyTemp(LinePoints),
			InParams.bEnabled ? ESlateDrawEffect::NoPixelSnapping : ESlateDrawEffect::NoPixelSnapping | ESlateDrawEffect::DisabledEffect,
			Color,
			4.f * InParams.Geometry.Scale,
			DashLength,
			CurTime*15.f);

		return InParams.Layer;

	}

	void MakeRotationExample( const FOnPaintHandlerParams& InParams )
	{
		const FSlateBrush* CenterBrush = FTestStyle::Get().GetBrush("TestRotation40px");
		const FSlateBrush* TestBrush = FTestStyle::Get().GetBrush("TestRotation20px");

		const FVector2D LocalPos = FVector2D(50,50);
		const FVector2D LocalSize = CenterBrush->ImageSize;

		FSlateLayoutTransform CenterLayoutTransform(LocalPos);

		// Make a box that rotates around its center.  Note if you don't specify the rotation point or rotation space
		// it defaults to rotating about the center of the box.  ERotationSpace::RelativeToElement is used by default in this case.  
		// If any rotation point is specified in that space, is relative to the element (0,0 is the upper left of the element)
		{
			FSlateDrawElement::MakeRotatedBox(
				InParams.OutDrawElements,
				InParams.Layer,
				InParams.Geometry.ToPaintGeometry( LocalSize, CenterLayoutTransform ),
				CenterBrush,
				InParams.bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
				CenterRotation);
		}

		// Make a box that rotates around the center of the previous box
		{
			FSlateLayoutTransform OrbitLayoutTransform(LocalPos + LocalSize);
			const FVector2D LocalCenterOfRotation = (LocalPos + (LocalPos+LocalSize))*.5f;
			const FVector2D RelativeOrbitPos = TransformPoint(Inverse(OrbitLayoutTransform), LocalCenterOfRotation);

			FSlateDrawElement::MakeRotatedBox(
				InParams.OutDrawElements,
				InParams.Layer,
				InParams.Geometry.ToPaintGeometry( TestBrush->ImageSize, OrbitLayoutTransform ),
				TestBrush,
				InParams.bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
				OuterRotation,
				RelativeOrbitPos);
		}
	}

	int32 TestRotation( const FOnPaintHandlerParams& InParams )
	{
		const FSlateBrush* StyleInfo = FCoreStyle::Get().GetBrush("FocusRectangle");

		FSlateDrawElement::MakeBox(
			InParams.OutDrawElements,
			InParams.Layer,
			InParams.Geometry.ToPaintGeometry(),
			StyleInfo,
			InParams.bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect
			);

		MakeRotationExample( InParams );

		return InParams.Layer;
	}

	int32 TestCustomVerts(const FOnPaintHandlerParams& InParams)
	{
		const float Radius = FMath::Min(InParams.Geometry.GetLocalSize().X, InParams.Geometry.GetLocalSize().Y) * 0.5f;
		const FVector2D Center = FVector2D(InParams.Geometry.AbsolutePosition) + InParams.Geometry.GetLocalSize() * 0.5f;

		const FSlateBrush* MyBrush = FCoreStyle::Get().GetBrush("ColorWheel.HueValueCircle");

		FSlateResourceHandle Handle = MyBrush->GetRenderingResource();
		const FSlateShaderResourceProxy* ResourceProxy = Handle.GetResourceProxy();

		FVector2f UVCenter = FVector2f::ZeroVector;
		FVector2f UVRadius = FVector2f(1,1);
		if (ResourceProxy != nullptr)
		{
			UVRadius = 0.5f * ResourceProxy->SizeUV;
			UVCenter = ResourceProxy->StartUV + UVRadius;
		}

		// Make a triangle fan in the area allotted
		const int NumTris = 12;
		TArray<FSlateVertex> Verts;
		Verts.Reserve(NumTris*3);

		// Center Vertex
		Verts.AddZeroed();
		{
			FSlateVertex& NewVert = Verts.Last();
			NewVert.Position[0] = Center.X;
			NewVert.Position[1] = Center.Y;
			NewVert.TexCoords[0] = UVCenter.X;
			NewVert.TexCoords[1] = UVCenter.Y;
			NewVert.TexCoords[2] = NewVert.TexCoords[3] = 1.0f;
			NewVert.Color = FColor::White;
		}

		for (int i = 0; i < NumTris; ++i)
		{
			Verts.AddZeroed();
			{
				const float Angle = (2*PI*i) / NumTris;
				const FVector2D EdgeDirection(FMath::Cos(Angle), FMath::Sin(Angle));
				const FVector2D Edge(Radius*EdgeDirection);
				FSlateVertex& NewVert = Verts.Last();
				NewVert.Position[0] = Center.X + Edge.X;
				NewVert.Position[1] = Center.Y + Edge.Y;
				NewVert.TexCoords[0] = UVCenter.X + UVRadius.X*EdgeDirection.X;
				NewVert.TexCoords[1] = UVCenter.Y + UVRadius.Y*EdgeDirection.Y;
				NewVert.TexCoords[2] = NewVert.TexCoords[3] = 1.0f;
				NewVert.Color = FColor::White;
			}
		}

		TArray<SlateIndex> Indexes;
		for (int i = 1; i <= NumTris; ++i)
		{
			Indexes.Add(0);
			Indexes.Add(i);
			Indexes.Add( (i+1 > 12) ? (1) : (i+1) );
		}

		FSlateDrawElement::MakeCustomVerts(InParams.OutDrawElements, InParams.Layer, Handle, Verts, Indexes, nullptr, 0, 0);

		return InParams.Layer;
	}
};


class SDocumentsTest : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SDocumentsTest ){}
	SLATE_END_ARGS()

	struct FDocumentInfo
	{
		FDocumentInfo( const FText& InDisplayName )
			: DisplayName( InDisplayName )
		{

		}

		FText DisplayName;
	};

	void Construct( const FArguments& InArgs, FTabManager* InTabManager )
	{
		TabManager = InTabManager;

		{
			Documents.Add( MakeShareable( new FDocumentInfo( LOCTEXT("Document01", "Document 1") ) ) );
			Documents.Add( MakeShareable( new FDocumentInfo( LOCTEXT("Document02", "Document 2") ) ) );
			Documents.Add( MakeShareable( new FDocumentInfo( LOCTEXT("Document03", "Document 3") ) ) );
			Documents.Add( MakeShareable( new FDocumentInfo( LOCTEXT("Document04", "Document 4") ) ) );
			Documents.Add( MakeShareable( new FDocumentInfo( LOCTEXT("Document05", "Document 5") ) ) );
		}

		bButtonOneVisible = true;
		
		this->ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SNew(SListView< TSharedRef<FDocumentInfo> >)
				.SelectionMode(ESelectionMode::None)
				.ListItemsSource( &Documents )
				.OnGenerateRow( this, &SDocumentsTest::GenerateListRow )
			]
			+ SVerticalBox::Slot() .HAlign(HAlign_Center) .VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(20).HAlign(HAlign_Center).VAlign(VAlign_Center)
				[
					SNew(SButton)
					.RenderTransform( FSlateRenderTransform( Concatenate( FVector2D(20.0f, 20.0f), FQuat2D(-PI/12.0f) ) ) )
					.ContentPadding(20).HAlign(HAlign_Center).VAlign(VAlign_Center)
					[
						SNew(SButton)
						.RenderTransform( FSlateRenderTransform( Concatenate( FVector2D(20.0f, 20.0f), FQuat2D(-PI/12.0f) ) ) )
						.ContentPadding(20).HAlign(HAlign_Center).VAlign(VAlign_Center)
						[
							SNew(SButton)
							.IsEnabled(false)
							.RenderTransform( FSlateRenderTransform( Concatenate( FVector2D(20.0f, 20.0f), FQuat2D(-PI/12.0f) ) ) )
							.ContentPadding(20).HAlign(HAlign_Center).VAlign(VAlign_Center)
							[
								SNew(SButton)
								.RenderTransform( FSlateRenderTransform( Concatenate( FVector2D(20.0f, 20.0f), FQuat2D(-PI/12.0f) ) ) )
								.ContentPadding(20).HAlign(HAlign_Center).VAlign(VAlign_Center)
								[
									SNew(STextBlock)
									.Text(NSLOCTEXT("x","ClickMe","ClickMe!"))
								]
							]
						]
					]
				]
			]
			+ SVerticalBox::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(SButton)
					.Visibility( this, &SDocumentsTest::GetButtonOneVisibility )
					.Text(LOCTEXT("Button1", "Button One"))
					.OnClicked(this, &SDocumentsTest::ToggleButtonOneVisibility)
				]
				+ SHorizontalBox::Slot()
				[
					SNew(SButton)
					.Text(LOCTEXT("Button2", "Button Two"))
					.OnClicked(this, &SDocumentsTest::ToggleButtonOneVisibility)
				]
			]
		];
	}

	TSharedRef<ITableRow> GenerateListRow(TSharedRef< FDocumentInfo > InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew( STableRow< TSharedRef<FDocumentInfo> >, OwnerTable )
			[
				SNew(SButton)
				.OnClicked( FOnClicked::CreateSP( this, &SDocumentsTest::SummonDocumentButtonClicked, InItem ) )
				.Text( InItem->DisplayName )
			];
	}

	FReply SummonDocumentButtonClicked( TSharedRef<FDocumentInfo> DocumentName )
	{
		TabManager->InsertNewDocumentTab
		(
			"DocTest", FTabManager::ESearchPreference::RequireClosedTab,
			SNew( SDockTab )
			.Label( DocumentName->DisplayName )
			.TabRole( ETabRole::DocumentTab )
			[
				SNew(SBox)
				.HAlign(HAlign_Center) .VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text( DocumentName->DisplayName )
				]
			]
			.TabWellContentRight()
			[
				SNew(STextBlock).Text( LOCTEXT("DocumentRightContentLabel", "Right Content") )
			]
		);

		return FReply::Handled();
	}

	EVisibility GetButtonOneVisibility() const
	{
		return bButtonOneVisible ? EVisibility::Visible : EVisibility::Collapsed;
	}

	FReply ToggleButtonOneVisibility()
	{
		bButtonOneVisible = !bButtonOneVisible;
		return FReply::Handled();
	}

	FTabManager* TabManager;

	TArray< TSharedRef<FDocumentInfo> > Documents;

	bool bButtonOneVisible;

};

class SSplitterTest : public SCompoundWidget
{
	/** Visibility state and mutators. */
	struct FVisibilityCycler
	{	
	public:	
		FVisibilityCycler()
			: TheVisibility(EVisibility::Visible)
		{
		}

		EVisibility GetVisibility() const
		{
			return TheVisibility;
		}

		FReply CycleVisibility()
		{
			TheVisibility = NextVisibilityState(TheVisibility);
			return FReply::Handled();
		}

	protected:
		static EVisibility NextVisibilityState( const EVisibility InVisibility )
		{
			if (InVisibility == EVisibility::Visible)
			{
				return EVisibility::Hidden;
			}
			else if (InVisibility == EVisibility::Hidden)
			{
				return EVisibility::Collapsed;
			}
			else if ( InVisibility == EVisibility::Collapsed )
			{
				return EVisibility::Visible;
			}
			else
			{
				return EVisibility::Visible;
			}
		}

		EVisibility TheVisibility;
	};

public:
	SLATE_BEGIN_ARGS( SSplitterTest ){}
	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 *
	 * @param InArgs   Declartion from which to construct the widget
	 */
	void Construct(const FArguments& InArgs)
	{
		this->ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5)
			.HAlign(HAlign_Center)
			[
				// BUTTONS for toggling layout visibility
				// (Arranged in a mini-layout version of the splitters below)
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SButton)
						.Text( LOCTEXT("Col0Row0Visibility", "Col0Row0 Visibility") )
						.OnClicked( Col0Row0Vis, &FVisibilityCycler::CycleVisibility )						
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SButton)
						.Text( LOCTEXT("Col0Row1Visibility", "Col0Row1 Visibility") )
						.OnClicked( Col0Row1Vis, &FVisibilityCycler::CycleVisibility )
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SButton)
						.Text( LOCTEXT("Col0Row2Visibility", "Col0Row2 Visibility") )
						.OnClicked( Col0Row2Vis, &FVisibilityCycler::CycleVisibility )
					]

				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.Text( LOCTEXT("CenterVisibility", "CenterVis Visibility") )
					.OnClicked( CenterVis, &FVisibilityCycler::CycleVisibility )
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SButton)
						.Text( LOCTEXT("Col2Row0Visibility", "Col2Row0 Visibility") )
						.OnClicked( Col2Row0Vis, &FVisibilityCycler::CycleVisibility )
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SButton)
						.Text( LOCTEXT("Col2Row1Visibility", "Col2Row1 Visibility") )
						.OnClicked( Col2Row1Vis, &FVisibilityCycler::CycleVisibility )
					]					
				]
			]
			+ SVerticalBox::Slot()
			. FillHeight(1)
			[
				// SPLITTERS
				SAssignNew(TopLeveSplitter, SSplitter)
				.ResizeMode( ESplitterResizeMode::Fill )
				+ SSplitter::Slot()
				[
					SAssignNew(Nested0, SSplitter)
						.Orientation(Orient_Vertical)
						.ResizeMode( ESplitterResizeMode::Fill )
						+SSplitter::Slot()
					[
						SNew(SBorder)
							.Visibility(Col0Row0Vis, &FVisibilityCycler::GetVisibility)
						[
							SNew(STextBlock)
								.Text( LOCTEXT("Col0Row0", "Col 0 Row 0") )
						]
					]
					+SSplitter::Slot()
					[
						SNew(SBorder)
							.Visibility(Col0Row1Vis, &FVisibilityCycler::GetVisibility)
						[
							SNew(STextBlock) .Text( LOCTEXT("Col0Row1", "Col 0 Row 1") )
						]
					]
					+SSplitter::Slot()
					[
						SNew(SBorder)
							.Visibility(Col0Row2Vis, &FVisibilityCycler::GetVisibility)
						[
							SNew(STextBlock) .Text( LOCTEXT("Col0Row2", "Col 0 Row 2") )
						]
					]
				]
				+ SSplitter::Slot()
				. SizeRule( SSplitter::SizeToContent )
				[
					SNew(SBorder)
						.Visibility(CenterVis, &FVisibilityCycler::GetVisibility)
						.Padding( 5.0f )
					[
						SNew(SButton)
							.OnClicked( this, &SSplitterTest::FlipTopLevelSplitter )
							.Text( LOCTEXT("Re-orient", "Re-orient") )
					]
				]
				+ SSplitter::Slot()
				[
					SAssignNew(Nested1, SSplitter)
						.Orientation(Orient_Vertical)
						.ResizeMode( ESplitterResizeMode::Fill )
						+SSplitter::Slot()
					[
						SNew(SBorder)
							.Visibility(Col2Row0Vis, &FVisibilityCycler::GetVisibility)
						[
							SNew(STextBlock) .Text( LOCTEXT("Col2Row0", "Col 2 Row 0") )
						]
					]
					+SSplitter::Slot()
					[
						SNew(SBorder)
							.Visibility(Col2Row1Vis, &FVisibilityCycler::GetVisibility)
						[
							SNew(STextBlock) .Text( LOCTEXT("Col2Row1", "Col 1 Row 1") )
						]
					]
				]
			]
		];
	}

	SSplitterTest()
		: Col0Row0Vis( new FVisibilityCycler() )
		, Col0Row1Vis( new FVisibilityCycler() )
		, Col0Row2Vis( new FVisibilityCycler() )
		, CenterVis( new FVisibilityCycler() )
		, Col2Row0Vis( new FVisibilityCycler() )
		, Col2Row1Vis( new FVisibilityCycler() )
	{
	}

protected:
	FReply FlipTopLevelSplitter()
	{
		TopLeveSplitter->SetOrientation( TopLeveSplitter->GetOrientation() == Orient_Horizontal ? Orient_Vertical : Orient_Horizontal );
		Nested0->SetOrientation( Nested0->GetOrientation() == Orient_Horizontal ? Orient_Vertical : Orient_Horizontal );
		Nested1->SetOrientation( Nested1->GetOrientation() == Orient_Horizontal ? Orient_Vertical : Orient_Horizontal );
		return FReply::Handled();
	}

	// Visibility states for the various cells within the splitter test.
	TSharedRef<FVisibilityCycler> Col0Row0Vis;
	TSharedRef<FVisibilityCycler> Col0Row1Vis;
	TSharedRef<FVisibilityCycler> Col0Row2Vis;

	TSharedRef<FVisibilityCycler> CenterVis;

	TSharedRef<FVisibilityCycler> Col2Row0Vis;
	TSharedRef<FVisibilityCycler> Col2Row1Vis;
		

	TSharedPtr<SSplitter> TopLeveSplitter;
	TSharedPtr<SSplitter> Nested0;
	TSharedPtr<SSplitter> Nested1;

};

#if WITH_FANCY_TEXT

class SMultiLineEditingTest : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( SMultiLineEditingTest )
	{}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs )
	{
		bIsReadOnly = false;
		MultilineEditableText = LOCTEXT( "MultiLineTextTest", "He has refused his Assent to Laws, the most wholesome and necessary for the public good.\nHe has forbidden his Governors to pass Laws of immediate and pressing importance, unless suspended in their operation till his Assent should be obtained; and when so suspended, he has utterly neglected to attend to them.\nHe has refused to pass other Laws for the accommodation of large districts of people, unless those people would relinquish the right of Representation in the Legislature, a right inestimable to them and formidable to tyrants only.\n\nHe has called together legislative bodies at places unusual, uncomfortable, and distant from the depository of their public Records, for the sole purpose of fatiguing them into compliance with his measures.\nHe has dissolved Representative Houses repeatedly, for opposing with manly firmness his invasions on the rights of the people.\nHe has refused for a long time, after such dissolutions, to cause others to be elected; whereby the Legislative powers, incapable of Annihilation, have returned to the People at large for their exercise; the State remaining in the mean time exposed to all the dangers of invasion from without, and convulsions within.\nHe has endeavoured to prevent the population of these States; for that purpose obstructing the Laws for Naturalization of Foreigners; refusing to pass others to encourage their migrations hither, and raising the conditions of new Appropriations of Lands.\n" );

		this->ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight( 1 )
			[
				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign( VAlign_Top )
				.Padding( 2 )
				[
					SNew( SVerticalBox )
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( STextBlock )
						.Text( LOCTEXT( "MultiLineTextWrapAt600px", "Multi-line editable text wrapping at 600px" ) )
					]
					+ SVerticalBox::Slot()
					[
						SNew( SMultiLineEditableTextBox )
						.Text( this, &SMultiLineEditingTest::GetMultilineEditableText )
						.Font( FCoreStyle::GetDefaultFontStyle("Regular", 12) )
						.Justification( ETextJustify::Center )
						.LineHeightPercentage( 2.0f )
						.OnTextCommitted( this, &SMultiLineEditingTest::HandleMultilineEditableTextCommitted )
						.WrapTextAt( 600.0f )
					]
				]
				+ SHorizontalBox::Slot()
				.FillWidth( 1 )
				.Padding( 2 )
				[
					SNew( SVerticalBox )
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( SHorizontalBox )
						
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew( STextBlock )
							.Text( LOCTEXT( "MultiLineTextAutoWrap", "Multi-line editable text auto-wrapping" ) )
						]

						+SHorizontalBox::Slot()
						.HAlign(HAlign_Right)
						[
							SNew( SCheckBox )
							.IsChecked(this, &SMultiLineEditingTest::IsReadOnlyChecked)
							.OnCheckStateChanged(this, &SMultiLineEditingTest::OnReadOnlyCheckedStateChanged)
							[
								SNew( STextBlock )
								.Text( LOCTEXT( "MultiLineTextReadOnly", "Read-only?" ) )
							]
						]
					]
					+ SVerticalBox::Slot()
					[
						SNew( SMultiLineEditableTextBox )
						.Margin( 10 )
						.Text( MultilineEditableText )
						.IsReadOnly( this, &SMultiLineEditingTest::IsReadOnly )
						//.Justification(ETextJustify::Right)
						.Font( FCoreStyle::GetDefaultFontStyle("Regular", 12) )
						.AutoWrapText( true )
						.HintText( LOCTEXT( "TypehereTextHint", "Type Here" ) )
					]
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew( SHorizontalBox )
				+ SHorizontalBox::Slot()
				.VAlign( VAlign_Top )
				.Padding( 2 )
				[
					SNew( SVerticalBox )
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( STextBlock )
						.Text( LOCTEXT( "MultiLineTextNoWrap", "Multi-line editable text without wrapping" ) )
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew( SHorizontalBox )
						+ SHorizontalBox::Slot()
						[
							SNew( SMultiLineEditableTextBox )
							.Font( FCoreStyle::GetDefaultFontStyle("Regular", 12) )
							.HintText( LOCTEXT("MultiLineHintText", "This is hint line 1\nThis is hint line 2") )
							//.WrapTextAt(300.0f)
							//.Justification( ETextJustify::Right )
						]
					]
				]
			]
		];
	}

private:
	FText GetMultilineEditableText() const
	{
		return MultilineEditableText;
	}

	void HandleMultilineEditableTextCommitted( const FText& Text, ETextCommit::Type Type )
	{
		MultilineEditableText = Text;
	}

	bool IsReadOnly() const
	{
		return bIsReadOnly;
	}

	ECheckBoxState IsReadOnlyChecked() const
	{
		return (bIsReadOnly) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void OnReadOnlyCheckedStateChanged(ECheckBoxState InState)
	{
		bIsReadOnly = (InState == ECheckBoxState::Checked);
	}

	bool bIsReadOnly;
	FText MultilineEditableText;
};


struct RichTextHelper
{
	static FText GetAliceInWonderland()
	{
		return FText::FromString(
			TEXT("The <a id=\"browser\" href=\"http://en.wikipedia.org/wiki/Dormouse_(Alice%27s_Adventures_in_Wonderland)\" style=\"RichText.Interactive.Text.Hyperlink\">Dormouse</> had closed its eyes by this time, and was going off into a doze; but, on being pinched by the Hatter, it woke up again with a little shriek, and went on: '<RichText.Interactive.Text.Dialogue>-that begins with an M, such as </><a id=\"browser\" href=\"http://en.wikipedia.org/wiki/Mousetrap\" style=\"RichText.Interactive.Text.DialogueHyperlink\">mouse-traps</><RichText.Interactive.Text.Dialogue>, and the moon, and memory, and muchness-you know you say things are \"much of a muchness\"-did you ever see such a thing as a drawing of a muchness?</>'")
			TEXT("\n\n")
			TEXT("'<RichText.Interactive.Text.Dialogue>Really, now you ask me,</>' said <a id=\"browser\" href=\"http://en.wikipedia.org/wiki/Alice_(Alice%27s_Adventures_in_Wonderland)\" style=\"RichText.Interactive.Text.Hyperlink\">Alice</>, very much confused, '<RichText.Interactive.Text.Dialogue>I don't think-</>'")
			TEXT("\n\n")
			TEXT("'<RichText.Interactive.Text.Dialogue>Then you shouldn't talk,</>' said the <a id=\"browser\" href=\"http://en.wikipedia.org/wiki/The_Hatter\" style=\"RichText.Interactive.Text.Hyperlink\">Hatter</>.")
			TEXT("\n\n")
			TEXT("This piece of rudeness was more than <a id=\"browser\" href=\"http://en.wikipedia.org/wiki/Alice_(Alice%27s_Adventures_in_Wonderland)\" style=\"RichText.Interactive.Text.Hyperlink\">Alice</> could bear: she got up in great disgust, and walked off; the <a id=\"browser\" href=\"http://en.wikipedia.org/wiki/Dormouse_(Alice%27s_Adventures_in_Wonderland)\" style=\"RichText.Interactive.Text.Hyperlink\">Dormouse</> fell asleep instantly, and neither of the others took the least notice of her going, though she looked back once or twice, half hoping that they would call after her: the last time she saw them, they were trying to put the <a id=\"browser\" href=\"http://en.wikipedia.org/wiki/Dormouse_(Alice%27s_Adventures_in_Wonderland)\" style=\"RichText.Interactive.Text.Hyperlink\">Dormouse</> into the teapot.")
			TEXT("\n\n")
			TEXT("'<RichText.Interactive.Text.Dialogue>At any rate I'll never go </><RichText.Interactive.Text.StrongDialogue>THERE</><RichText.Interactive.Text.Dialogue> again!</>' said <a id=\"browser\" href=\"http://en.wikipedia.org/wiki/Alice_(Alice%27s_Adventures_in_Wonderland)\" style=\"RichText.Interactive.Text.Hyperlink\">Alice</> as she picked her way through the wood. '<RichText.Interactive.Text.Dialogue>It's the stupidest tea-party I ever was at in all my life!</>'")
			TEXT("\n\n")
			TEXT("Just as she said this, she noticed that one of the trees had a door leading right into it. '<RichText.Interactive.Text.Dialogue>That's very curious!</>' she thought. '<RichText.Interactive.Text.Dialogue>But everything's curious today. I think I may as well go in at once.</>' And in she went.")
			TEXT("\n\n")
			TEXT("Once more she found herself in the long hall, and close to the little glass table. '<RichText.Interactive.Text.Dialogue>Now, I'll manage better this time,</>' she said to herself, and began by taking the little golden key, and unlocking the door that led into the garden. Then she went to work nibbling at the mushroom (she had kept a piece of it in her pocket) till she was about a foot high: then she walked down the little passage: and THEN-she found herself at last in the beautiful garden, among the bright flower-beds and the cool fountains.")
			TEXT("\n\n")
			TEXT("A large rose-tree stood near the entrance of the garden: the roses growing on it were white, but there were three gardeners at it, busily painting them red. <a id=\"browser\" href=\"http://en.wikipedia.org/wiki/Alice_(Alice%27s_Adventures_in_Wonderland)\" style=\"RichText.Interactive.Text.Hyperlink\">Alice</> thought this a very curious thing, and she went nearer to watch them, and just as she came up to them she heard one of them say, '<RichText.Interactive.Text.Dialogue>Look out now, Five! Don't go splashing paint over me like that!</>'")
			TEXT("\n\n")
			TEXT("'<RichText.Interactive.Text.Dialogue>I couldn't help it,</>' said Five, in a sulky tone; '<RichText.Interactive.Text.Dialogue>Seven jogged my elbow.</>'")
			TEXT("\n\n")
			TEXT("On which Seven looked up and said, '<RichText.Interactive.Text.Dialogue>That's right, Five! Always lay the blame on others!</>'")
			TEXT("\n\n"));
	}

	static FText GetTheWarOfTheWorlds_Part1()
	{
		return FText::FromString(
			TEXT("When we had finished eating we went softly upstairs to my study, and I looked again out of the open window.  In one night the valley had become a valley of ashes.  The fires had dwindled now.  Where flames had been there were now streamers of smoke; but the countless ruins of shattered and gutted houses and blasted and blackened trees that the night had hidden stood out now gaunt and terrible in the pitiless light of dawn.  Yet here and there some object had had the luck to escape--a white railway signal here, the end of a greenhouse there, white and fresh amid the wreckage.  Never before in the history of warfare had destruction been so indiscriminate and so universal. And shining with the growing light of the east, three of the metallic giants stood about the pit, their cowls rotating as though they were surveying the desolation they had made.")
			TEXT("\n\n")
			TEXT("It seemed to me that the pit had been enlarged, and ever and again puffs of vivid green vapour streamed up and out of it towards the brightening dawn--streamed up, whirled, broke, and vanished.")
			TEXT("\n\n")
			TEXT("Beyond were the pillars of fire about Chobham.  They became pillars of bloodshot smoke at the first touch of day.")
			TEXT("\n\n")
			TEXT("As the dawn grew brighter we withdrew from the window from which we had watched the Martians, and went very quietly downstairs.")
			TEXT("\n\n")
			TEXT("The artilleryman agreed with me that the house was no place to stay in.  He proposed, he said, to make his way Londonward, and thence rejoin his battery--No. 12, of the Horse Artillery.  My plan was to return at once to Leatherhead; and so greatly had the strength of the Martians impressed me that I had determined to take my wife to Newhaven, and go with her out of the country forthwith.  For I already perceived clearly that the country about London must inevitably be the scene of a disastrous struggle before such creatures as these could be destroyed.")
			TEXT("\n\n")
			TEXT("Between us and Leatherhead, however, lay the third cylinder, with its guarding giants.  Had I been alone, I think I should have taken my chance and struck across country.  But the artilleryman dissuaded me: \"It's no kindness to the right sort of wife,\" he said, \"to make her a widow\"; and in the end I agreed to go with him, under cover of the woods, northward as far as Street Cobham before I parted with him. Thence I would make a big detour by Epsom to reach Leatherhead.")
			TEXT("\n\n")
			TEXT("I should have started at once, but my companion had been in active service and he knew better than that.  He made me ransack the house for a flask, which he filled with whiskey; and we lined every available pocket with packets of biscuits and slices of meat.  Then we crept out of the house, and ran as quickly as we could down the ill-made road by which I had come overnight.  The houses seemed deserted. In the road lay a group of three charred bodies close together, struck dead by the Heat-Ray; and here and there were things that people had dropped--a clock, a slipper, a silver spoon, and the like poor valuables.  At the corner turning up towards the post office a little cart, filled with boxes and furniture, and horseless, heeled over on a broken wheel.  A cash box had been hastily smashed open and thrown under the debris.")
			TEXT("\n\n")
			TEXT("Except the lodge at the Orphanage, which was still on fire, none of the houses had suffered very greatly here.  The Heat-Ray had shaved the chimney tops and passed.  Yet, save ourselves, there did not seem to be a living soul on Maybury Hill.  The majority of the inhabitants had escaped, I suppose, by way of the Old Woking road--the road I had taken when I drove to Leatherhead--or they had hidden.")
			TEXT("\n\n")
			TEXT("We went down the lane, by the body of the man in black, sodden now from the overnight hail, and broke into the woods at the foot of the hill.  We pushed through these towards the railway without meeting a soul.  The woods across the line were but the scarred and blackened ruins of woods; for the most part the trees had fallen, but a certain proportion still stood, dismal grey stems, with dark brown foliage instead of green.")
			TEXT("\n\n")
			TEXT("On our side the fire had done no more than scorch the nearer trees; it had failed to secure its footing.  In one place the woodmen had been at work on Saturday; trees, felled and freshly trimmed, lay in a clearing, with heaps of sawdust by the sawing-machine and its engine. Hard by was a temporary hut, deserted.  There was not a breath of wind this morning, and everything was strangely still.  Even the birds were hushed, and as we hurried along I and the artilleryman talked in whispers and looked now and again over our shoulders.  Once or twice we stopped to listen.")
			TEXT("\n\n"));
	}

	static FText GetTheWarOfTheWorlds_Part2()
	{
		return FText::FromString(
			TEXT("And beyond, over the blue hills that rise southward of the river, the glittering Martians went to and fro, calmly and methodically spreading their poison cloud over this patch of country and then over that, laying it again with their steam jets when it had served its purpose, and taking possession of the conquered country.  They do not seem to have aimed at extermination so much as at complete demoralisation and the destruction of any opposition.  They exploded any stores of powder they came upon, cut every telegraph, and wrecked the railways here and there.  They were hamstringing mankind.  They seemed in no hurry to extend the field of their operations, and did not come beyond the central part of London all that day.  It is possible that a very considerable number of people in London stuck to their houses through Monday morning.  Certain it is that many died at home suffocated by the Black Smoke.")
			TEXT("\n\n")
			TEXT("Until about midday the Pool of London was an astonishing scene. Steamboats and shipping of all sorts lay there, tempted by the enormous sums of money offered by fugitives, and it is said that many who swam out to these vessels were thrust off with boathooks and drowned.  About one o'clock in the afternoon the thinning remnant of a cloud of the black vapour appeared between the arches of Blackfriars Bridge.  At that the Pool became a scene of mad confusion, fighting, and collision, and for some time a multitude of boats and barges jammed in the northern arch of the Tower Bridge, and the sailors and lightermen had to fight savagely against the people who swarmed upon them from the riverfront.  People were actually clambering down the piers of the bridge from above.")
			TEXT("\n\n")
			TEXT("When, an hour later, a Martian appeared beyond the Clock Tower and waded down the river, nothing but wreckage floated above Limehouse.")
			TEXT("\n\n")
			TEXT("Of the falling of the fifth cylinder I have presently to tell.  The sixth star fell at Wimbledon.  My brother, keeping watch beside the women in the chaise in a meadow, saw the green flash of it far beyond the hills.  On Tuesday the little party, still set upon getting across the sea, made its way through the swarming country towards Colchester. The news that the Martians were now in possession of the whole of London was confirmed.  They had been seen at Highgate, and even, it was said, at Neasden.  But they did not come into my brother's view until the morrow.")
			TEXT("\n\n")
			TEXT("That day the scattered multitudes began to realise the urgent need of provisions.  As they grew hungry the rights of property ceased to be regarded.  Farmers were out to defend their cattle-sheds, granaries, and ripening root crops with arms in their hands.  A number of people now, like my brother, had their faces eastward, and there were some desperate souls even going back towards London to get food. These were chiefly people from the northern suburbs, whose knowledge of the Black Smoke came by hearsay.  He heard that about half the members of the government had gathered at Birmingham, and that enormous quantities of high explosives were being prepared to be used in automatic mines across the Midland counties.")
			TEXT("\n\n")
			TEXT("He was also told that the Midland Railway Company had replaced the desertions of the first day's panic, had resumed traffic, and was running northward trains from St. Albans to relieve the congestion of the home counties.  There was also a placard in Chipping Ongar announcing that large stores of flour were available in the northern towns and that within twenty-four hours bread would be distributed among the starving people in the neighbourhood.  But this intelligence did not deter him from the plan of escape he had formed, and the three pressed eastward all day, and heard no more of the bread distribution than this promise.  Nor, as a matter of fact, did anyone else hear more of it.  That night fell the seventh star, falling upon Primrose Hill.  It fell while Miss Elphinstone was watching, for she took that duty alternately with my brother.  She saw it.")
			TEXT("\n\n")
			TEXT("On Wednesday the three fugitives--they had passed the night in a field of unripe wheat--reached Chelmsford, and there a body of the inhabitants, calling itself the Committee of Public Supply, seized the pony as provisions, and would give nothing in exchange for it but the promise of a share in it the next day.  Here there were rumours of Martians at Epping, and news of the destruction of Waltham Abbey Powder Mills in a vain attempt to blow up one of the invaders.")
			TEXT("\n\n")
			TEXT("People were watching for Martians here from the church towers.  My brother, very luckily for him as it chanced, preferred to push on at once to the coast rather than wait for food, although all three of them were very hungry.  By midday they passed through Tillingham, which, strangely enough, seemed to be quite silent and deserted, save for a few furtive plunderers hunting for food.  Near Tillingham they suddenly came in sight of the sea, and the most amazing crowd of shipping of all sorts that it is possible to imagine.")
			TEXT("\n\n"));
	}

	static FText GetTheWarOfTheWorlds_Part3()
	{
		return FText::FromString(
			TEXT("They saw the gaunt figures separating and rising out of the water as they retreated shoreward, and one of them raised the camera-like generator of the Heat-Ray.  He held it pointing obliquely downward, and a bank of steam sprang from the water at its touch.  It must have driven through the iron of the ship's side like a white-hot iron rod through paper.")
			TEXT("\n\n")
			TEXT("But no one heeded that very much.  At the sight of the Martian's collapse the captain on the bridge yelled inarticulately, and all the crowding passengers on the steamer's stern shouted together.  And then they yelled again.  For, surging out beyond the white tumult, drove something long and black, the flames streaming from its middle parts, its ventilators and funnels spouting fire.")
			TEXT("\n\n")
			TEXT("She was alive still; the steering gear, it seems, was intact and her engines working.  She headed straight for a second Martian, and was within a hundred yards of him when the Heat-Ray came to bear.  Then with a violent thud, a blinding flash, her decks, her funnels, leaped upward.  The Martian staggered with the violence of her explosion, and in another moment the flaming wreckage, still driving forward with the impetus of its pace, had struck him and crumpled him up like a thing of cardboard.  My brother shouted involuntarily.  A boiling tumult of steam hid everything again.")
			TEXT("\n\n")
			TEXT("\"Two!\" yelled the captain.")
			TEXT("\n\n")
			TEXT("Everyone was shouting.  The whole steamer from end to end rang with frantic cheering that was taken up first by one and then by all in the crowding multitude of ships and boats that was driving out to sea.")
			TEXT("\n\n")
			TEXT("The little vessel continued to beat its way seaward, and the ironclads receded slowly towards the coast, which was hidden still by a marbled bank of vapour, part steam, part black gas, eddying and combining in the strangest way.  The fleet of refugees was scattering to the northeast; several smacks were sailing between the ironclads and the steamboat.  After a time, and before they reached the sinking cloud bank, the warships turned northward, and then abruptly went about and passed into the thickening haze of evening southward.  The coast grew faint, and at last indistinguishable amid the low banks of clouds that were gathering about the sinking sun.")
			TEXT("\n\n")
			TEXT("Then suddenly out of the golden haze of the sunset came the vibration of guns, and a form of black shadows moving.  Everyone struggled to the rail of the steamer and peered into the blinding furnace of the west, but nothing was to be distinguished clearly.  A mass of smoke rose slanting and barred the face of the sun.  The steamboat throbbed on its way through an interminable suspense.")
			TEXT("\n\n")
			TEXT("The sun sank into grey clouds, the sky flushed and darkened, the evening star trembled into sight.  It was deep twilight when the captain cried out and pointed.  My brother strained his eyes. Something rushed up into the sky out of the greyness--rushed slantingly upward and very swiftly into the luminous clearness above the clouds in the western sky; something flat and broad, and very large, that swept round in a vast curve, grew smaller, sank slowly, and vanished again into the grey mystery of the night.  And as it flew it rained down darkness upon the land.")
			TEXT("\n\n"));
	}

	static FText GetAroundTheWorldIn80Days_Rainbow()
	{
		return FText::FromString(
			TEXT("<Rainbow.Text.Red>\"</><Rainbow.Text.Orange>I</> <Rainbow.Text.Yellow>know</> <Rainbow.Text.Green>it;</> <Rainbow.Text.Blue>I</> <Rainbow.Text.Red>don't</> <Rainbow.Text.Orange>blame</> <Rainbow.Text.Yellow>you.</>  <Rainbow.Text.Green>We</> <Rainbow.Text.Blue>start</> <Rainbow.Text.Red>for</> <Rainbow.Text.Orange>Dover</> <Rainbow.Text.Yellow>and</> <Rainbow.Text.Green>Calais</> <Rainbow.Text.Blue>in</> <Rainbow.Text.Red>ten</> <Rainbow.Text.Orange>minutes.</>\"")
			TEXT("\n\n")
			TEXT("<Rainbow.Text.Yellow>A</> <Rainbow.Text.Green>puzzled</> <Rainbow.Text.Blue>grin</> <Rainbow.Text.Red>overspread</> <Rainbow.Text.Orange>Passepartout's</> <Rainbow.Text.Yellow>round</> <Rainbow.Text.Green>face;</> <Rainbow.Text.Blue>clearly</> <Rainbow.Text.Red>he</> <Rainbow.Text.Orange>had</> <Rainbow.Text.Yellow>not</> <Rainbow.Text.Green>comprehended</> <Rainbow.Text.Blue>his</> <Rainbow.Text.Red>employer.</>")
			TEXT("\n\n")
			TEXT("<Rainbow.Text.Orange>\"</><Rainbow.Text.Yellow>Monsieur</> <Rainbow.Text.Green>is</> <Rainbow.Text.Blue>going</> <Rainbow.Text.Red>to</> <Rainbow.Text.Orange>leave</> <Rainbow.Text.Yellow>home?</><Rainbow.Text.Green>\"</>")
			TEXT("\n\n")
			TEXT("<Rainbow.Text.Blue>\"</><Rainbow.Text.Red>Yes,</><Rainbow.Text.Orange>\"</> <Rainbow.Text.Yellow>returned</> <Rainbow.Text.Green>Phileas</> <Rainbow.Text.Blue>Fogg.</>  <Rainbow.Text.Red>\"</><Rainbow.Text.Orange>We</> <Rainbow.Text.Yellow>are</> <Rainbow.Text.Green>going</> <Rainbow.Text.Blue>round</> <Rainbow.Text.Red>the</> <Rainbow.Text.Yellow>world.</><Rainbow.Text.Green>\"</>")
			TEXT("\n\n")
			TEXT("<Rainbow.Text.Blue>Passepartout</> <Rainbow.Text.Red>opened</> <Rainbow.Text.Orange>wide</> <Rainbow.Text.Yellow>his</> <Rainbow.Text.Green>eyes,</> <Rainbow.Text.Blue>raised</> <Rainbow.Text.Red>his</> <Rainbow.Text.Orange>eyebrows,</> <Rainbow.Text.Yellow>held</> <Rainbow.Text.Green>up</> <Rainbow.Text.Blue>his</> <Rainbow.Text.Red>hands,</> <Rainbow.Text.Orange>and</> <Rainbow.Text.Yellow>seemed</> <Rainbow.Text.Green>about</> <Rainbow.Text.Blue>to</> <Rainbow.Text.Red>collapse,</> <Rainbow.Text.Orange>so</> <Rainbow.Text.Yellow>overcome</> <Rainbow.Text.Green>was</> <Rainbow.Text.Blue>he</> <Rainbow.Text.Red>with</> <Rainbow.Text.Orange>stupefied</> <Rainbow.Text.Yellow>astonishment.</>")
			TEXT("\n\n")
			TEXT("<Rainbow.Text.Green>\"</><Rainbow.Text.Blue>Round</> <Rainbow.Text.Red>the</> <Rainbow.Text.Orange>world!</><Rainbow.Text.Yellow>\"</> <Rainbow.Text.Green>he</> <Rainbow.Text.Blue>murmured.</>")
			TEXT("\n\n")
			TEXT("<Rainbow.Text.Red>\"</><Rainbow.Text.Orange>In</> <Rainbow.Text.Yellow>eighty</> <Rainbow.Text.Green>days,</><Rainbow.Text.Blue>\"</> <Rainbow.Text.Red>responded</> <Rainbow.Text.Yellow>Mr. Fogg.</>  <Rainbow.Text.Green>\"</><Rainbow.Text.Blue>So</> <Rainbow.Text.Red>we</> <Rainbow.Text.Yellow>haven't</> <Rainbow.Text.Green>a</> <Rainbow.Text.Blue>moment</> <Rainbow.Text.Red>to</> <Rainbow.Text.Orange>lose.</><Rainbow.Text.Yellow>\"</>")
			TEXT("\n\n")
			TEXT("<Rainbow.Text.Green>\"</><Rainbow.Text.Blue>But</> <Rainbow.Text.Red>the</> <Rainbow.Text.Yellow>trunks?</><Rainbow.Text.Green>\"</> <Rainbow.Text.Blue>gasped</> <Rainbow.Text.Red>Passepartout,</> <Rainbow.Text.Yellow>unconsciously</> <Rainbow.Text.Green>swaying</> <Rainbow.Text.Blue>his</> <Rainbow.Text.Red>head</> <Rainbow.Text.Yellow>from</> <Rainbow.Text.Green>right</> <Rainbow.Text.Blue>to</> <Rainbow.Text.Red>left.</>")
			TEXT("<Rainbow.Text.Orange>\"</><Rainbow.Text.Yellow>We'll</> <Rainbow.Text.Green>have</> <Rainbow.Text.Blue>no</> <Rainbow.Text.Red>trunks;</> <Rainbow.Text.Orange>only</> <Rainbow.Text.Yellow>a</> <Rainbow.Text.Green>carpet-bag,</> <Rainbow.Text.Blue>with</> <Rainbow.Text.Red>two</> <Rainbow.Text.Yellow>shirts</> <Rainbow.Text.Green>and</> <Rainbow.Text.Blue>three</> <Rainbow.Text.Red>pairs</> <Rainbow.Text.Orange>of</> <Rainbow.Text.Yellow>stockings</> <Rainbow.Text.Green>for</> <Rainbow.Text.Blue>me,</> <Rainbow.Text.Red>and</> <Rainbow.Text.Orange>the</> <Rainbow.Text.Yellow>same</> <Rainbow.Text.Green>for</> <Rainbow.Text.Blue>you.</>  <Rainbow.Text.Red>We'll</> <Rainbow.Text.Orange>buy</> <Rainbow.Text.Yellow>our</> <Rainbow.Text.Green>clothes</> <Rainbow.Text.Blue>on</> <Rainbow.Text.Red>the</> <Rainbow.Text.Orange>way.</>  <Rainbow.Text.Yellow>Bring</> <Rainbow.Text.Green>down</> <Rainbow.Text.Blue>my</> <Rainbow.Text.Red>mackintosh</> <Rainbow.Text.Orange>and</> <Rainbow.Text.Yellow>traveling-cloak,</> <Rainbow.Text.Green>and</> <Rainbow.Text.Blue>some</> <Rainbow.Text.Red>stout</> <Rainbow.Text.Yellow>shoes,</> <Rainbow.Text.Green>though</> <Rainbow.Text.Blue>we</> <Rainbow.Text.Red>shall</> <Rainbow.Text.Orange>do</> <Rainbow.Text.Yellow>little</> <Rainbow.Text.Green>walking.</>  <Rainbow.Text.Blue>Make haste!</><Rainbow.Text.Red>\"</>")
			TEXT("\n\n")
			TEXT("<Rainbow.Text.Orange>Passepartout</> <Rainbow.Text.Yellow>tried</> <Rainbow.Text.Green>to</> <Rainbow.Text.Blue>reply,</> <Rainbow.Text.Red>but</> <Rainbow.Text.Orange>could</> <Rainbow.Text.Yellow>not.</>  <Rainbow.Text.Green>He</> <Rainbow.Text.Blue>went</> <Rainbow.Text.Red>out,</> <Rainbow.Text.Orange>mounted</> <Rainbow.Text.Yellow>to</> <Rainbow.Text.Green>his</> <Rainbow.Text.Blue>own</> <Rainbow.Text.Red>room,</> <Rainbow.Text.Orange>fell</> <Rainbow.Text.Yellow>into</> <Rainbow.Text.Green>a</> <Rainbow.Text.Blue>chair,</> <Rainbow.Text.Red>and</> <Rainbow.Text.Orange>muttered:</> <Rainbow.Text.Yellow>\"</><Rainbow.Text.Green>That's</> <Rainbow.Text.Blue>good,</> <Rainbow.Text.Red>that</> <Rainbow.Text.Orange>is!</> <Rainbow.Text.Yellow>And</> <Rainbow.Text.Green>I,</> <Rainbow.Text.Blue>who</> <Rainbow.Text.Red>wanted</> <Rainbow.Text.Orange>to</> <Rainbow.Text.Yellow>remain</> <Rainbow.Text.Green>quiet!</><Rainbow.Text.Blue>\"</>")
			TEXT("\n\n"));
	}

	static void OnBrowserLinkClicked(const FSlateHyperlinkRun::FMetadata& Metadata, TSharedRef<SWidget> ParentWidget)
	{
		const FString* url = Metadata.Find(TEXT("href"));

		if(url)
		{
			FPlatformProcess::LaunchURL(**url, nullptr, nullptr);
		}
		else
		{
			SpawnProClickerPopUp(LOCTEXT("FailedToFindUrlPopUpMessage", "Sorry this hyperlink is not <RichText.Tagline.TextHighlight>configured incorrectly</>!"), ParentWidget);
		}
	}

	static void SpawnProClickerPopUp(const FText& Text, TSharedRef<SWidget> ParentWidget)
	{
		TSharedRef<SWidget> Widget =
			SNew(SBorder).Padding(10).BorderImage(FTestStyle::Get().GetBrush("RichText.Tagline.Background"))
			[
				SNew(SRichTextBlock)
				.Text(Text)
				.TextStyle(FTestStyle::Get(), "RichText.Tagline.Text")
				.DecoratorStyleSet(&FTestStyle::Get())
				.Justification(ETextJustify::Center)
			];


		FSlateApplication::Get().PushMenu(
			ParentWidget, // Parent widget should be TestSuite, not the menu thats open or it will be closed when the menu is dismissed
			FWidgetPath(),
			Widget,
			FSlateApplication::Get().GetCursorPos(), // summon location
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
			);
	}
};

class SRichTextTest : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SRichTextTest ){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		WrapWidth = 600;
		bShouldWrap = true;
		LineHeight = 1.0f;

		Margin = FMargin( 20 );

		JustificationTypeOptions.Empty();
		JustificationTypeOptions.Add( MakeShareable( new FString( "Left" ) ) );
		JustificationTypeOptions.Add( MakeShareable( new FString( "Center" ) ) );
		JustificationTypeOptions.Add( MakeShareable( new FString( "Right" ) ) );
		Justification = ETextJustify::Left;

		//// Drop shadow border
		//SNew(SBorder)
		//	.Padding( 5.0f )
		//	.BorderImage( FCoreStyle::Get().GetBrush("BoxShadow") )
		//	[

		this->ChildSlot
		[
			SNew( SScrollBox )
			+SScrollBox::Slot()
			[
				SNew( SBorder )
				.BorderImage( FTestStyle::Get().GetBrush( "RichText.Background" ) )
				[
					SNew( SVerticalBox )
					+ SVerticalBox::Slot().AutoHeight() .Padding(0)			
					[
						SNew( SBorder )
						.BorderImage( FTestStyle::Get().GetBrush( "RichText.Tagline.Background" ) )
						.Padding(0)
						[
							SNew( SRichTextBlock )
							.Text( LOCTEXT("RichTextHeader05", "This is a text heavy page that has been created to show the performance and capabilities of Slate's <RichText.Tagline.TextHighlight>SRichTextBlock</>.") )
							.TextStyle( FTestStyle::Get(), "RichText.Tagline.Text" )
							.DecoratorStyleSet( &FTestStyle::Get() )
							.WrapTextAt( 800 )
							.Justification( ETextJustify::Center )
							.Margin( FMargin(20) )
						]
					]

					+ SVerticalBox::Slot().AutoHeight() .HAlign(HAlign_Center) .Padding(20)			
					[
						SNew( SRichTextBlock )
						.Text( LOCTEXT("RichText.HowItWorks", "<RichText.Header>What does it do?</>\n\nThe SRichTextBlock uses a concept called Decorators to introduce new font styles, images, animation and even whole interactive widgets inline with the text.\n\nSlate comes with a couple Decorators out of the box:\n\n        \u2022  <RichText.Text.Fancy>Text</> Decorator\n        \u2022  <img src=\"RichText.ImageDecorator\"/> Image Decorator\n        \u2022  <RichText.WidgetDecorator>Widget</> Decorator\n        \u2022  <a id=\"HyperlinkDecorator\" style=\"RichText.Hyperlink\">Hyperlink</> Decorator\n\n<RichText.Header>What about options?</>\n\nOf course you can always introduce your own Decorators by implementing ITextDecorator. This will give you full customization in how your text or widget is styled. \n\nBesides the power of Decorators the SRichTextBlock comes equipped with Margin support, Left-Center-Right Text Justification, a Line Height Scalar and Highlighting.\n\n<RichText.Header>How does the markup work?</>\n\nWell the markup parser is customizable so you can adjust the markup anyway you'd like by providing your own parser.  The parser that comes with Slate though uses a syntax very similar to xml.\n\n        \u2022  &lt;TextBlockStyleName>Your text content&lt;/>\n        \u2022  &lt;img src=\"SlateBrushStyleName\"/>\n        \u2022  &lt;a id=\"YourCustomId\"/>Your hyperlink text&lt;/>\n\nIf you ever want to use the markup syntax as actual text you can escape the markup using xml style escapes. For example:\n\n        \u2022  <    &amp;lt;\n        \u2022  >    &amp;gt;\n        \u2022  \"     &amp;quot;\n\nBut you only need to escape these characters when a set of them match actual syntax so this isn't generally an issue.\n\n<RichText.Header>Are there any catches?</>\n\nThere are still plenty of things the SRichTextBlock doesn't currently support. The most notable lacking feature is not having the ability to flow text around images or widgets. ") )
						.TextStyle( FTestStyle::Get(), "RichText.Text" )
						.DecoratorStyleSet( &FTestStyle::Get() )
						.WrapTextAt( 600 )
						+ SRichTextBlock::ImageDecorator()
						+ SRichTextBlock::HyperlinkDecorator( TEXT("HyperlinkDecorator"), this, &SRichTextTest::OnHyperlinkDecoratorClicked )
						+ SRichTextBlock::WidgetDecorator( TEXT("RichText.WidgetDecorator"), this, &SRichTextTest::OnCreateWidgetDecoratorWidget )
					]

					+ SVerticalBox::Slot().AutoHeight() .Padding(0)			
					[
						SNew( SBorder )
						.BorderImage( FTestStyle::Get().GetBrush( "RichText.Tagline.Background" ) )
						.Padding(0)
						[
							SNew( SRichTextBlock )
							.Text( LOCTEXT("RichTextHeader01", "Here is an <RichText.Tagline.TextHighlight>interactive example</> of the different <RichText.Tagline.TextHighlight>SRichTextBlock</> features in action!") )
							.TextStyle( FTestStyle::Get(), "RichText.Tagline.Text" )
							.DecoratorStyleSet( &FTestStyle::Get() )
							.WrapTextAt( 800 )
							.Justification( ETextJustify::Center )
							.Margin( FMargin(20) )
						]
					]

					+ SVerticalBox::Slot().AutoHeight() .HAlign(HAlign_Center) .Padding(20)		
					[
						SNew( SHorizontalBox )
						+SHorizontalBox::Slot() .Padding( 5, 0 )
						[
							SNew( SBorder )
							.BorderImage( FTestStyle::Get().GetBrush( "RichText.Interactive.Details.Background" ) )
							.Padding( 10 )
							[
								SNew( SBox )
								.WidthOverride( 400 )
								[
									SNew( SGridPanel )
									.FillColumn( 1, 1.0f )
									+ SGridPanel::Slot( 0, 0 ) .ColumnSpan( 2 ) .VAlign(VAlign_Center)
									[
										SNew( SRichTextBlock ) .Text( LOCTEXT("RichText.MarginHeader", "Margin") ) .TextStyle( FTestStyle::Get(), "RichText.Interactive.Details.Name.Text" )
									]

									+ SGridPanel::Slot( 0, 1 ) .Padding( 20,0,5,0 ) .VAlign(VAlign_Center)
									[
										SNew( SRichTextBlock ) .Text( LOCTEXT("RichText.LeftMargin", "Left") ) .TextStyle( FTestStyle::Get(), "RichText.Interactive.Details.Name.Text" )
									]

									+ SGridPanel::Slot( 1, 1 ) .Padding(0, 5)
									[
										SNew( SSpinBox<float> )
										.MinValue(0.0f)
										.MaxValue(65536.0f)
										.MaxSliderValue(100.0f)
										.Delta(0.25f)
										.Value(this, &SRichTextTest::GetLeftMargin)
										.OnValueChanged(this, &SRichTextTest::SetLeftMargin)
										.Font( FTestStyle::Get().GetFontStyle("RichText.Interactive.Details.Value.Text") )
									]

									+ SGridPanel::Slot( 0, 2 ) .Padding( 20,0,5,0 ) .VAlign(VAlign_Center)
									[
										SNew( SRichTextBlock ) .Text( LOCTEXT("RichText.TopMargin", "Top") ) .TextStyle( FTestStyle::Get(), "RichText.Interactive.Details.Name.Text" )
									]

									+ SGridPanel::Slot( 1, 2 ) .Padding(0, 5)
									[
										SNew( SSpinBox<float> )
										.MinValue(0.0f)
										.MaxValue(65536.0f)
										.MaxSliderValue(100.0f)
										.Delta(0.25f)
										.Value(this, &SRichTextTest::GetTopMargin)
										.OnValueChanged(this, &SRichTextTest::SetTopMargin)
										.Font( FTestStyle::Get().GetFontStyle("RichText.Interactive.Details.Value.Text") )
									]

									+ SGridPanel::Slot( 0, 3 ) .Padding( 20,0,5,0 ) .VAlign(VAlign_Center)
									[
										SNew( SRichTextBlock ) .Text( LOCTEXT("RichText.RightMargin", "Right") ) .TextStyle( FTestStyle::Get(), "RichText.Interactive.Details.Name.Text" )
									]

									+ SGridPanel::Slot( 1, 3 ) .Padding(0, 5)
									[
										SNew( SSpinBox<float> )
										.MinValue(0.0f)
										.MaxValue(65536.0f)
										.MaxSliderValue(100.0f)
										.Delta(0.25f)
										.Value(this, &SRichTextTest::GetRightMargin)
										.OnValueChanged(this, &SRichTextTest::SetRightMargin)
										.Font( FTestStyle::Get().GetFontStyle("RichText.Interactive.Details.Value.Text") )
									]

									+ SGridPanel::Slot( 0, 4 ) .Padding( 20,0,5,0 ) .VAlign(VAlign_Center)
									[
										SNew( SRichTextBlock ) .Text( LOCTEXT("RichText.BottomMargin", "Bottom") ) .TextStyle( FTestStyle::Get(), "RichText.Interactive.Details.Name.Text" )
									]

									+ SGridPanel::Slot( 1, 4 ) .Padding(0, 5)
									[
										SNew( SSpinBox<float> )
										.MinValue(0.0f)
										.MaxValue(65536.0f)
										.MaxSliderValue(100.0f)
										.Delta(0.25f)
										.Value(this, &SRichTextTest::GetBottomMargin)
										.OnValueChanged(this, &SRichTextTest::SetBottomMargin)
										.Font( FTestStyle::Get().GetFontStyle("RichText.Interactive.Details.Value.Text") )
									]

									+ SGridPanel::Slot( 0, 5 ) .Padding( 0,0,5,0 ) .VAlign(VAlign_Center)
									[
										SNew( SRichTextBlock ) .Text( LOCTEXT("RichText.ShouldWrap", "Should Wrap") ) .TextStyle( FTestStyle::Get(), "RichText.Interactive.Details.Name.Text" )
									]

									+ SGridPanel::Slot( 1, 5 ) .Padding(0, 5)
									.ColumnSpan(2)
									[
										SNew( SCheckBox )
										.IsChecked( this, &SRichTextTest::ShouldWrapRichText )
										.OnCheckStateChanged( this, &SRichTextTest::ShouldWrapRichTextChanged )
										.Style( FTestStyle::Get(), "RichText.Interactive.Details.Checkbox" )
									]

									+ SGridPanel::Slot( 0, 6 ) .Padding( 0,0,5,0 ) .VAlign(VAlign_Center)
									[
										SNew( SRichTextBlock ) .Text( LOCTEXT("RichText.WrapWidth", "Wrap Width") ) .TextStyle( FTestStyle::Get(), "RichText.Interactive.Details.Name.Text" )
									]

									+ SGridPanel::Slot( 1, 6 ) .Padding(0, 5)
									[
										SNew( SSpinBox<float> )
										.MinValue(1.0f)
										.MaxValue(800.0f)
										.MinSliderValue(1.0f)
										.MaxSliderValue(800.0f)
										.Delta(1.0f)
										.Value(this, &SRichTextTest::GetWrapWidth)
										.OnValueChanged(this, &SRichTextTest::SetWrapWidth)
										.Font( FTestStyle::Get().GetFontStyle("RichText.Interactive.Details.Value.Text") )
									]

									+ SGridPanel::Slot( 0, 7 ) .Padding( 0,0,5,0 ) .VAlign(VAlign_Center)
									[
										SNew( SRichTextBlock ) .Text( LOCTEXT("RichText.TextJustify", "Text Justify") ) .TextStyle( FTestStyle::Get(), "RichText.Interactive.Details.Name.Text" )
									]

									+ SGridPanel::Slot( 1, 7 ) .Padding(0, 5)
									[
										SNew( SComboBox< TSharedPtr< FString > > )
										.OptionsSource( &JustificationTypeOptions )
										.OnSelectionChanged( this, &SRichTextTest::JustificationComboBoxSelectionChanged )
										.OnGenerateWidget( this, &SRichTextTest::MakeWidgetFromJustificationOption )
										[
											SNew( STextBlock ) .Text( this, &SRichTextTest::JustificationGetSelectedText ) .TextStyle( FTestStyle::Get(), "RichText.Interactive.Details.Value.Text" )
										]
									]

									+ SGridPanel::Slot( 0, 8 ) .Padding( 0,0,5,0 ) .VAlign(VAlign_Center)
									[
										SNew( SRichTextBlock ) .Text( LOCTEXT("RichText.LineHeight", "Line Height") ) .TextStyle( FTestStyle::Get(), "RichText.Interactive.Details.Name.Text" )
									]

									+ SGridPanel::Slot( 1, 8 ) .Padding(0, 5)
									[
										SNew( SSpinBox<float> )
										.MinValue(0.1f)
										.MaxValue(5.0f)
										.MinSliderValue(0.1f)
										.MaxSliderValue(5)
										.Value(this, &SRichTextTest::GetLineHeight)
										.OnValueChanged(this, &SRichTextTest::SetLineHeight)
										.Font( FTestStyle::Get().GetFontStyle("RichText.Interactive.Details.Value.Text") )
									]

								]
							]
						]
						+SHorizontalBox::Slot() .AutoWidth() .HAlign(HAlign_Center)
						[
							SNew( SBox ).WidthOverride( 800 ) .HAlign(HAlign_Center)
							[
								SNew( SVerticalBox )
								+ SVerticalBox::Slot().AutoHeight() .HAlign(HAlign_Right) .VAlign(VAlign_Center) .Padding( 5, 5 )
								[
									SNew( SHorizontalBox )
									+ SHorizontalBox::Slot() .MaxWidth( 300 )
									[
										SNew( SSearchBox )
										.OnTextChanged( this, &SRichTextTest::OnTextChanged )
									]
								]

								+ SVerticalBox::Slot()
								[
									SNew(SBorder)
									.Padding( 5.0f )
									.BorderImage( FCoreStyle::Get().GetBrush("BoxShadow") )
									[
										SNew( SBorder ) .Padding( 2 ) .HAlign(HAlign_Center)
										.BorderImage( FTestStyle::Get().GetBrush( "RichText.Background" ) )
										[
											SAssignNew( InteractiveRichText, SRichTextBlock )
											.Text( RichTextHelper::GetAliceInWonderland() )
											.TextStyle( FTestStyle::Get(), "RichText.Interactive.Text" )
											.DecoratorStyleSet( &FTestStyle::Get() )
											.Margin( this, &SRichTextTest::GetRichTextMargin )
											.WrapTextAt( this, &SRichTextTest::GetRichTextWrapWidthValue )
											.Justification( this, &SRichTextTest::JustificationGetSelected )
											.LineHeightPercentage( this, &SRichTextTest::GetLineHeight )
											+ SRichTextBlock::HyperlinkDecorator( TEXT("browser"), FSlateHyperlinkRun::FOnClick::CreateStatic( &RichTextHelper::OnBrowserLinkClicked, AsShared() ) )
										]
									]
								]
							]
						]
						+SHorizontalBox::Slot() .FillWidth( 1.0f )
					]

					+ SVerticalBox::Slot().AutoHeight() .Padding(0)			
					[
						SNew( SBorder )
						.BorderImage( FTestStyle::Get().GetBrush( "RichText.Tagline.Background" ) )
						.Padding(0)
						[
							SNew( SRichTextBlock)
							.Text( LOCTEXT("RichTextHeader02", "Here's a bunch of text just to <RichText.Tagline.TextHighlight>measure performance</>.") )
							.TextStyle( FTestStyle::Get(), "RichText.Tagline.Text" )
							.DecoratorStyleSet( &FTestStyle::Get() )
							.WrapTextAt( 800 )
							.Justification( ETextJustify::Center )
							.Margin( FMargin(20) )
						]
					]

					+ SVerticalBox::Slot().AutoHeight() .HAlign(HAlign_Center) .Padding(20)			
					[
						SNew( SHorizontalBox )
						+ SHorizontalBox::Slot()
						[
							SNew( SRichTextBlock)
							.Text( RichTextHelper::GetTheWarOfTheWorlds_Part1() )
							.TextStyle( FTestStyle::Get(), "TheWarOfTheWorlds.Text" )
							.DecoratorStyleSet( &FTestStyle::Get() )
							.WrapTextAt( 266 )
						]

						+ SHorizontalBox::Slot() .Padding( 25, 0 )
						[
							SNew( SRichTextBlock)
							.Text( RichTextHelper::GetTheWarOfTheWorlds_Part2() )
							.TextStyle( FTestStyle::Get(), "TheWarOfTheWorlds.Text" )
							.DecoratorStyleSet( &FTestStyle::Get() )
							.WrapTextAt( 266 )
						]

						+ SHorizontalBox::Slot()
						[
							SNew( SRichTextBlock)
							.Text( RichTextHelper::GetTheWarOfTheWorlds_Part3() )
							.TextStyle( FTestStyle::Get(), "TheWarOfTheWorlds.Text" )
							.DecoratorStyleSet( &FTestStyle::Get() )
							.WrapTextAt( 266 )
						]
					]

					+ SVerticalBox::Slot().AutoHeight() .Padding(0)			
					[
						SNew( SBorder )
						.BorderImage( FTestStyle::Get().GetBrush( "RichText.Tagline.Background" ) )
						.Padding(0)
						[
							SNew( SRichTextBlock)
							.Text( LOCTEXT("RichTextHeader03", "Here's a bunch of text in rainbow colors!\n<RichText.Tagline.SubtleText>Also to measure </><RichText.Tagline.SubtleTextHighlight>performance</><RichText.Tagline.SubtleText>, why else?</> ") )
							.TextStyle( FTestStyle::Get(), "RichText.Tagline.Text" )
							.DecoratorStyleSet( &FTestStyle::Get() )
							.WrapTextAt( 800 )
							.Justification( ETextJustify::Center )
							.Margin( FMargin(20) )
						]
					]

					+ SVerticalBox::Slot().AutoHeight() .HAlign(HAlign_Center) .Padding(20)			
					[
						SNew( SHorizontalBox )
						+ SHorizontalBox::Slot()
						[
							SNew( SRichTextBlock)
							.Text( RichTextHelper::GetAroundTheWorldIn80Days_Rainbow() )
							.TextStyle( FTestStyle::Get(), "Rainbow.Text" )
							.DecoratorStyleSet( &FTestStyle::Get() )
							.WrapTextAt( 600 )
						]
					]

					+ SVerticalBox::Slot().AutoHeight() .Padding(0)			
					[
						SNew( SBox )
						.WidthOverride( 800 )
						.HeightOverride( 200 )
						[
							SNew( SBorder )
							.BorderImage( FTestStyle::Get().GetBrush( "RichText.Tagline.Background" ) )
							.Padding(0)
							[
								SNew( SRichTextBlock)
								.Text( LOCTEXT("RichTextHeader04", "That's all <RichText.Tagline.TextHighlight>folks</>. Hope you enjoyed this page about <RichText.Tagline.TextHighlight>SRichTextBlock</>!") )
								.TextStyle( FTestStyle::Get(), "RichText.Tagline.Text" )
								.DecoratorStyleSet( &FTestStyle::Get() )
								.WrapTextAt( 800 )
								.Justification( ETextJustify::Center )
								.Margin( FMargin(20) )
							]
						]
					]
				]
			]
		];
	}

	FSlateWidgetRun::FWidgetRunInfo OnCreateWidgetDecoratorWidget( const FTextRunInfo& RunInfo, const ISlateStyle* Style ) const
	{
		TSharedRef< SWidget > Widget = SNew( SButton ) .OnClicked( const_cast<SRichTextTest*>(this), &SRichTextTest::OnWidgetDecoratorClicked ) 
			.ToolTip( 
				SNew( SToolTip )
				.BorderImage( FTestStyle::Get().GetBrush( "RichText.Tagline.Background" ) )
				[
					SNew( SRichTextBlock )
					.Text( LOCTEXT("WidgetDecoratorTooltip", " With the <RichText.TextHighlight>Widget Decorator</> you can <RichText.TextHighlight>inline any widget</> in your text!") )
					.TextStyle( FTestStyle::Get(), "RichText.Text" )
					.DecoratorStyleSet( &FTestStyle::Get() )
				]
			)
			[
				SNew( SRichTextBlock )
				.Text( RunInfo.Content )
				.TextStyle( FTestStyle::Get(), "RichText.Text" )
				.DecoratorStyleSet( &FTestStyle::Get() )
			];
		
		TSharedRef< FSlateFontMeasure > FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		int16 Baseline = FontMeasure->GetBaseline( FTestStyle::Get().GetWidgetStyle<FTextBlockStyle>( "RichText.Text" ).Font );

		return FSlateWidgetRun::FWidgetRunInfo( Widget, Baseline - 3 );
	}

	FReply OnWidgetDecoratorClicked()
	{
		RichTextHelper::SpawnProClickerPopUp( LOCTEXT("WidgetDecoratorExamplePopUpMessage", "I don't really do anything. <RichText.Tagline.TextHighlight>Sorry</>."), AsShared() );
		return FReply::Handled();
	}

	void OnHyperlinkDecoratorClicked( const FSlateHyperlinkRun::FMetadata& Metadata )
	{
		RichTextHelper::SpawnProClickerPopUp( LOCTEXT("HyperlinkDecoratorExamplePopUpMessage", "You're a <RichText.Tagline.TextHighlight>pro</> at clicking!"), AsShared() );
	}

	void OnTextChanged( const FText& NewText )
	{
		InteractiveRichText->SetHighlightText( NewText );
	}

	float GetLeftMargin() const
	{
		return Margin.Left;
	}

	void SetLeftMargin( float Value )
	{
		Margin.Left = Value;
	}

	float GetTopMargin() const
	{
		return Margin.Top;
	}

	void SetTopMargin( float Value )
	{
		Margin.Top = Value;
	}

	float GetRightMargin() const
	{
		return Margin.Right;
	}

	void SetRightMargin( float Value )
	{
		Margin.Right = Value;
	}

	float GetBottomMargin() const
	{
		return Margin.Bottom;
	}

	void SetBottomMargin( float Value )
	{
		Margin.Bottom = Value;
	}

	FMargin GetRichTextMargin() const
	{
		return Margin;
	}

	TSharedRef<SWidget> MakeWidgetFromJustificationOption( TSharedPtr< FString > Value )
	{
		return SNew(STextBlock) .Text( JustificationGetText( Value ) );
	}

	void JustificationComboBoxSelectionChanged( TSharedPtr< FString > Value, ESelectInfo::Type SelectInfo )
	{
		if ( *Value == TEXT("Left") )
		{
			Justification = ETextJustify::Left;
		}
		else if ( *Value == TEXT("Center") )
		{
			Justification = ETextJustify::Center;
		}
		else if ( *Value == TEXT("Right") )
		{
			Justification = ETextJustify::Right;
		}
		else if ( *Value == TEXT("Invariant Left") )
		{
			Justification = ETextJustify::InvariantLeft;
		}
		else if ( *Value == TEXT("Invariant Right") )
		{
			Justification = ETextJustify::InvariantRight;
		}
	}

	FText JustificationGetSelectedText() const
	{
		FText Text;
		if ( Justification == ETextJustify::Left )
		{
			Text = LOCTEXT("TextJustify::Left", "Left");
		}
		else if ( Justification == ETextJustify::Center )
		{
			Text = LOCTEXT("TextJustify::Center", "Center");
		}
		else if ( Justification == ETextJustify::Right )
		{
			Text = LOCTEXT("TextJustify::Right", "Right");
		}
		else if ( Justification == ETextJustify::InvariantLeft )
		{
			Text = LOCTEXT("TextJustify::InvariantLeft", "Invariant Left");
		}
		else if ( Justification == ETextJustify::InvariantRight )
		{
			Text = LOCTEXT("TextJustify::InvariantRight", "Invariant Right");
		}
		return Text;
	}

	ETextJustify::Type JustificationGetSelected() const
	{
		return Justification;
	}

	FText JustificationGetText( TSharedPtr< FString > Value ) const
	{
		FText Text;
		if ( *Value == TEXT("Left") )
		{
			Text = LOCTEXT("TextJustify::Left", "Left");
		}
		else if ( *Value == TEXT("Center") )
		{
			Text = LOCTEXT("TextJustify::Center", "Center");
		}
		else if ( *Value == TEXT("Right") )
		{
			Text = LOCTEXT("TextJustify::Right", "Right");
		}

		return Text;
	}

	float GetWrapWidth() const
	{
		return WrapWidth;
	}

	void SetWrapWidth( float Value )
	{
		WrapWidth = Value;
	}

	ECheckBoxState ShouldWrapRichText() const
	{
		return bShouldWrap ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void ShouldWrapRichTextChanged( ECheckBoxState CheckState )
	{
		bShouldWrap = CheckState == ECheckBoxState::Checked ? true : false;
	}

	float GetRichTextWrapWidthValue() const
	{
		if ( bShouldWrap )
		{
			return WrapWidth;
		}

		return 0;
	}

	float GetLineHeight() const
	{
		return LineHeight;
	}

	void SetLineHeight( float NewValue )
	{
		LineHeight = NewValue;
	}

private:

	float WrapWidth;
	bool bShouldWrap;

	FMargin Margin;
	ETextJustify::Type Justification;
	TArray< TSharedPtr< FString > > JustificationTypeOptions;

	float LineHeight;

	TSharedPtr< SRichTextBlock > InteractiveRichText;
};

#endif //WITH_FANCY_TEXT


class STextEditTest : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( STextEditTest ){}
	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 *
	 * @param InArgs   Declaration from which to construct the widget
	 */
	void Construct(const FArguments& InArgs)
	{
		bIsPassword = true;

		InlineEditableText = LOCTEXT( "TestingInlineEditableTextBlock", "Testing inline editable text block!" );

		Animation = FCurveSequence(0, 5);
		
		Animation.Play(this->AsShared(), true);

		this->ChildSlot
		[
			SNew( SScrollBox )
			+SScrollBox::Slot()
			[
				SNew( SVerticalBox )
				+ SVerticalBox::Slot().AutoHeight() .HAlign(HAlign_Center) .Padding(5)			
				[
					SAssignNew( EditableText, SEditableText )
					.Text( LOCTEXT( "TestingTextControl", "Testing editable text control (no box)" ) )
					.Font( FCoreStyle::GetDefaultFontStyle("Regular", 12) )
					.HintText(LOCTEXT("TestingTextControlHint", "Hint Text"))
				]
				+ SVerticalBox::Slot().AutoHeight() .HAlign(HAlign_Center) .Padding(5)
				[
					SNew( SEditableTextBox )
					.Text( LOCTEXT( "TestingReadOnlyTextBox", "Read only editable text box (with tool tip!)" ) )
					.Font( FCoreStyle::GetDefaultFontStyle("Regular", 20) )
					.IsReadOnly( true )
					.ToolTipText( LOCTEXT("TestingReadOnlyTextBox_Tooltip", "Testing tool tip for editable text!") )
					.HintText(LOCTEXT("TestingReadOnlyTextBoxHint", "Hint Text") )
				]
				+ SVerticalBox::Slot().AutoHeight() .HAlign(HAlign_Center) .Padding(5)
				[
					SNew( SEditableTextBox )
					.Text( LOCTEXT( "TestingLongText", "Here is an editable text box with a very long initial string.  Useful to test scrolling.  Remember, this editable text box has many features, such as cursor navigation, text selection with either the mouse or keyboard, and cut, copy and paste.  You can even undo and redo just how you'd expect to." ) )
					.Font( FCoreStyle::GetDefaultFontStyle("Bold", 13) )
					.HintText(LOCTEXT("TestingLongTextHint", "Hint Text"))
				]
				+ SVerticalBox::Slot().AutoHeight() .HAlign(HAlign_Center) .Padding(5)
				[
					SNew( SEditableTextBox )
					.Text( LOCTEXT( "TestingBigTextBigMargin", "Big text, big margin!" ) )
					.Font( FCoreStyle::GetDefaultFontStyle("Bold", 40) )
					.RevertTextOnEscape( true )
					.BackgroundColor( this, &STextEditTest::GetLoopingColor )
					.HintText(LOCTEXT("TestingBigTextMarginHint", "Hint Text"))
				]
				+ SVerticalBox::Slot().AutoHeight() .HAlign(HAlign_Center) .Padding(5)			
				[
					SAssignNew(InlineEditableTextBlock, SInlineEditableTextBlock)
					.Font( FCoreStyle::GetDefaultFontStyle("Regular", 12) )
					.Text( InlineEditableText )
					.OnTextCommitted( this, &STextEditTest::InlineEditableTextCommited )
					.ToolTipText( LOCTEXT("TestingInlineEditableTextBlock_Tooltip", "Testing tool tip for inline editable text block!") )
				]
				+ SVerticalBox::Slot().AutoHeight() .HAlign(HAlign_Center) .VAlign(VAlign_Center) .Padding( 5 )
				[
					SAssignNew( SearchBox, SSearchBox )
					//.Text( LOCTEXT("TestingSearchBox", "Testing search boxes tool tip") )
					//.Font( FCoreStyle::GetDefaultFontStyle("Bold", 12) )
					.ToolTipText( LOCTEXT("TestingSearchBox_Tooltip", "Testing search boxes") )
				]
				+ SVerticalBox::Slot().AutoHeight() .HAlign(HAlign_Center) .VAlign(VAlign_Center) .Padding( 5 )
				[
					SAssignNew( NumericInput, SEditableTextBox )
					.Text( LOCTEXT( "NumericInput", "This should be a number" ) )
					.OnTextChanged( this, &STextEditTest::OnNumericInputTextChanged )
					.RevertTextOnEscape( true )
					.HintText(LOCTEXT("NumericInputHint", "Enter a number"))
				]
				+ SVerticalBox::Slot().AutoHeight() .HAlign(HAlign_Center) .VAlign(VAlign_Center) .Padding( 5 )
				[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						. HAlign(HAlign_Center)
						[
							SNew(SButton)
								.Text( LOCTEXT("PopupTest", "PopUp Test") )
								.OnClicked( this, &STextEditTest::LaunchPopUp_OnClicked )
						]
				]
				+SVerticalBox::Slot().AutoHeight() .VAlign(VAlign_Bottom) .Padding(0,20,0,0)
				[
					SAssignNew(ErrorText, SErrorText)
				]
				+ SVerticalBox::Slot().AutoHeight() .HAlign(HAlign_Center) .VAlign(VAlign_Center) .Padding( 5 )
				[
					SNew( SEditableTextBox )
					.Text( LOCTEXT( "CustomContextMenuInput", "This text box has a custom context menu" ) )
					.RevertTextOnEscape( true )
					.HintText(LOCTEXT("CustomContextMenuHint", "Custom context menu..."))
					.OnContextMenuOpening( this, &STextEditTest::OnCustomContextMenuOpening )
				]
				+ SVerticalBox::Slot().AutoHeight() .HAlign(HAlign_Center) .VAlign(VAlign_Center) .Padding( 5 )
				[
					SNew( SEditableTextBox )
					.Text( LOCTEXT( "DisabledContextMenuInput", "This text box has no context menu" ) )
					.RevertTextOnEscape( true )
					.HintText(LOCTEXT("DisabledContextMenuHint", "No context menu..."))
					.OnContextMenuOpening(this, &STextEditTest::OnDisabledContextMenuOpening)
				]
				+ SVerticalBox::Slot().AutoHeight() .HAlign(HAlign_Center) .VAlign(VAlign_Center) .Padding( 5 )
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SEditableTextBox)
						.IsPassword(this, &STextEditTest::IsPassword)
						.RevertTextOnEscape(true)
						.MinDesiredWidth(400)
						.HintText(LOCTEXT("EditablePasswordHintText", "This text box can be a password"))
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(4, 0))
					[
						SNew(SCheckBox)
						.IsChecked(this, &STextEditTest::GetPasswordCheckState)
						.OnCheckStateChanged(this, &STextEditTest::OnPasswordCheckStateChanged)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("PasswordCheckBoxText", "Password?"))
						]
					]
				]
			]
		];
	}

	void FocusDefaultWidget()
	{
		// Set focus to the editable text, so the user doesn't have to click initially to type
		FWidgetPath WidgetToFocusPath;
		FSlateApplication::Get().GeneratePathToWidgetUnchecked( EditableText.ToSharedRef(), WidgetToFocusPath );
		FSlateApplication::Get().SetKeyboardFocus( WidgetToFocusPath, EFocusCause::SetDirectly );
	}

	void InlineEditableTextCommited(const FText& NewText, ETextCommit::Type CommitType )
	{
		InlineEditableText = NewText;
		InlineEditableTextBlock->SetText( InlineEditableText );
	}

	void OnNumericInputTextChanged( const FText& NewText )
	{
		const FText Error = (NewText.IsNumeric())
			? FText::GetEmpty()
			: FText::Format( LOCTEXT("NotANumberWarning", "'{0}' is not a number"), NewText );

		ErrorText->SetError( Error );
		NumericInput->SetError( Error );
	}

	TSharedPtr<SWidget> OnCustomContextMenuOpening()
	{
		return SNew(SBorder)
			.Padding(5.0f)
			.BorderImage(FCoreStyle::Get().GetBrush("BoxShadow"))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CustomContextMenuContent", "This context menu content is just a text block"))
			];
	}

	TSharedPtr<SWidget> OnDisabledContextMenuOpening()
	{
		return TSharedPtr<SWidget>();
	}

	bool IsPassword() const
	{
		return bIsPassword;
	}

	ECheckBoxState GetPasswordCheckState() const
	{
		return bIsPassword ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void OnPasswordCheckStateChanged(ECheckBoxState NewState)
	{
		bIsPassword = NewState == ECheckBoxState::Checked;
	}

	void ClearSearchBox()
	{
		SearchBox->SetText( FText::GetEmpty() );
	}

	FSlateColor GetLoopingColor() const
	{
		return FLinearColor( 360*Animation.GetLerp(), 0.8f, 1.0f).HSVToLinearRGB();
	}

	FReply LaunchPopUp_OnClicked ()
	{		
		FText DefaultText( LOCTEXT("EnterThreeChars", "Enter a three character string") );

		TSharedRef<STextEntryPopup> TextEntry = SAssignNew( PopupInput, STextEntryPopup )
			.Label( DefaultText )
			.ClearKeyboardFocusOnCommit ( false )
			.OnTextChanged( this, &STextEditTest::OnPopupTextChanged )
			.OnTextCommitted( this, &STextEditTest::OnPopupTextCommitted ) 
			.HintText( DefaultText );

		PopupMenu = FSlateApplication::Get().PushMenu(
			AsShared(), // Parent widget should be TestSyuite, not the menu thats open or it will be closed when the menu is dismissed
			FWidgetPath(),
			TextEntry,
			FSlateApplication::Get().GetCursorPos(), // summon location
			FPopupTransitionEffect( FPopupTransitionEffect::TypeInPopup )
		);

		return FReply::Handled();
	}

	void OnPopupTextChanged (const FText& NewText)
	{
		const FText Error = ( NewText.ToString().Len() == 3 )
			? FText::GetEmpty()
			: FText::Format( LOCTEXT("ThreeCharsError", "'{0}' is not three characters"), NewText);

		ErrorText->SetError( Error );
		PopupInput->SetError( Error );
	}

	void OnPopupTextCommitted( const FText& NewText, ETextCommit::Type CommitInfo )
	{
		if ( (CommitInfo == ETextCommit::OnEnter) && (NewText.ToString().Len() == 3) )
		{
			// manually close menu on validated committal
			if (PopupMenu.IsValid())
			{		
				PopupMenu.Pin()->Dismiss();
			}
		}
	}
protected:

	TSharedPtr< SEditableText > EditableText;

	TSharedPtr<SEditableTextBox> SearchBox;

	FCurveSequence Animation;

	TSharedPtr<SErrorText> ErrorText;

	TSharedPtr<SEditableTextBox> NumericInput;

#if WITH_FANCY_TEXT

	TSharedPtr<SRichTextBlock> RichTextBlock;

#endif //WITH_FANCY_TEXT

	TSharedPtr<STextEntryPopup> PopupInput;

	TWeakPtr<IMenu> PopupMenu;

	TSharedPtr<SInlineEditableTextBlock> InlineEditableTextBlock;
	FText InlineEditableText;

	bool bIsPassword;
 };

 #if WITH_FANCY_TEXT

 /**
  * This is used in conjunction with the TextStyle decorator to allow arbitrary styling of text within a rich-text editor
  * This struct defines a set of known font families, as well as providing some utility functions for converting the text style to and from a text layout run
  */
struct FTextStyles
{
	/** Flags controlling which TTF or OTF font should be picked from the given font family */
	struct EFontStyle
	{
		typedef uint8 Flags;
		enum Flag
		{
			Regular = 0,
			Bold = 1<<0,
			Italic = 1<<1,
		};
	};

	/** This struct defines a font family, which combines multiple TTF or OTF fonts into a single group, allowing text to be styled as bold or italic */
	struct FFontFamily
	{
		FFontFamily(FText InDisplayName, FName InFamilyName, const TSharedRef<const FCompositeFont>& InCompositeFont)
			: DisplayName(InDisplayName)
			, FamilyName(InFamilyName)
			, CompositeFont(InCompositeFont)
		{
		}

		/** Name used for this font family in the UI */
		FText DisplayName;

		/** Named used to identify this family from the TextStyle decorator */
		FName FamilyName;

		/** Composite font to use (should contain at least a Regular, Bold, Italic, and BoldItalic style) */
		TSharedRef<const FCompositeFont> CompositeFont;
	};

	/** Get the font style name to use based on the requested style flags */
	static FString GetFontStyleString(const EFontStyle::Flags InFontStyle)
	{
		FString FontStyleString;
		if(InFontStyle == EFontStyle::Regular)
		{
			FontStyleString = TEXT("Regular");
		}
		else
		{
			if(InFontStyle & EFontStyle::Bold)
			{
				FontStyleString += TEXT("Bold");
			}
			if(InFontStyle & EFontStyle::Italic)
			{
				FontStyleString += TEXT("Italic");
			}
		}
		return FontStyleString;
	}

	/** Convert the given text style into run meta-information, so that valid source rich-text formatting can be generated for it */
	static FRunInfo CreateRunInfo(const TSharedPtr<FFontFamily>& InFontFamily, const uint16 InFontSize, const EFontStyle::Flags InFontStyle, const FLinearColor& InFontColor)
	{
		FRunInfo RunInfo(TEXT("TextStyle"));
		RunInfo.MetaData.Add(TEXT("FontFamily"), InFontFamily->FamilyName.ToString());
		RunInfo.MetaData.Add(TEXT("FontSize"), FString::FromInt(InFontSize));
		RunInfo.MetaData.Add(TEXT("FontStyle"), GetFontStyleString(InFontStyle));
		RunInfo.MetaData.Add(TEXT("FontColor"), InFontColor.ToString());
		return RunInfo;
	}

	/** Explode some run meta-information back out into its component text style parts */
	void ExplodeRunInfo(const FRunInfo& InRunInfo, TSharedPtr<FFontFamily>& OutFontFamily, uint16& OutFontSize, EFontStyle::Flags& OutFontStyle, FLinearColor& OutFontColor) const
	{
		check(AvailableFontFamilies.Num());

		const FString* const FontFamilyString = InRunInfo.MetaData.Find(TEXT("FontFamily"));
		if(FontFamilyString)
		{
			OutFontFamily = FindFontFamily(FName(**FontFamilyString));
		}
		if(!OutFontFamily.IsValid())
		{
			OutFontFamily = AvailableFontFamilies[0];
		}

		OutFontSize = 11;
		const FString* const FontSizeString = InRunInfo.MetaData.Find(TEXT("FontSize"));
		if(FontSizeString)
		{
			OutFontSize = static_cast<uint16>(FPlatformString::Atoi(**FontSizeString));
		}

		OutFontStyle = EFontStyle::Regular;
		const FString* const FontStyleString = InRunInfo.MetaData.Find(TEXT("FontStyle"));
		if(FontStyleString)
		{
			if(*FontStyleString == TEXT("Bold"))
			{
				 OutFontStyle = EFontStyle::Bold;
			}
			else if(*FontStyleString == TEXT("Italic"))
			{
				 OutFontStyle = EFontStyle::Italic;
			}
			else if(*FontStyleString == TEXT("BoldItalic"))
			{
				 OutFontStyle = EFontStyle::Bold | EFontStyle::Italic;
			}
		}

		OutFontColor = FLinearColor::Black;
		const FString* const FontColorString = InRunInfo.MetaData.Find(TEXT("FontColor"));
		if(FontColorString && !OutFontColor.InitFromString(*FontColorString))
		{
			OutFontColor = FLinearColor::Black;
		}
	}

	/** Convert the given text style into a text block style for use by Slate */
	static FTextBlockStyle CreateTextBlockStyle(const TSharedPtr<FFontFamily>& InFontFamily, const uint16 InFontSize, const EFontStyle::Flags InFontStyle, const FLinearColor& InFontColor)
	{
		FSlateFontInfo FontInfo;
		FontInfo.CompositeFont = InFontFamily->CompositeFont;
		FontInfo.TypefaceFontName = *GetFontStyleString(InFontStyle);
		FontInfo.Size = InFontSize;

		FTextBlockStyle TextBlockStyle;
		TextBlockStyle.SetFont(FontInfo);
		TextBlockStyle.SetColorAndOpacity(InFontColor);
		return TextBlockStyle;
	}

	/** Convert the given run meta-information into a text block style for use by Slate */
	FTextBlockStyle CreateTextBlockStyle(const FRunInfo& InRunInfo) const
	{
		TSharedPtr<FFontFamily> FontFamily;
		uint16 FontSize;
		EFontStyle::Flags FontStyle;
		FLinearColor FontColor;
		ExplodeRunInfo(InRunInfo, FontFamily, FontSize, FontStyle, FontColor);
		return CreateTextBlockStyle(FontFamily, FontSize, FontStyle, FontColor);
	}

	/** Try and find a font family with the given name */
	TSharedPtr<FFontFamily> FindFontFamily(const FName InFamilyName) const
	{
		const TSharedPtr<FFontFamily>* const FoundFontFamily = AvailableFontFamilies.FindByPredicate([InFamilyName](TSharedPtr<FFontFamily>& Entry) -> bool
		{
			return Entry->FamilyName == InFamilyName;
		});
		return (FoundFontFamily) ? *FoundFontFamily : nullptr;
	}

	TArray<TSharedPtr<FFontFamily>> AvailableFontFamilies;
};

/**
 * This is a custom decorator used to allow arbitrary styling of text within a rich-text editor
 * This is required since normal text styling can only work with known styles from a given Slate style-set
 */
class FTextStyleDecoratorTest : public ITextDecorator
{
public:

	static TSharedRef<FTextStyleDecoratorTest> Create(FTextStyles* const InTextStyles)
	{
		return MakeShareable(new FTextStyleDecoratorTest(InTextStyles));
	}

	virtual ~FTextStyleDecoratorTest()
	{
	}

	virtual bool Supports(const FTextRunParseResults& RunParseResult, const FString& Text) const override
	{
		return (RunParseResult.Name == TEXT("TextStyle"));
	}

	virtual TSharedRef<ISlateRun> Create(const TSharedRef<FTextLayout>& TextLayout, const FTextRunParseResults& RunParseResult, const FString& OriginalText, const TSharedRef<FString>& InOutModelText, const ISlateStyle* Style) override
	{
		FRunInfo RunInfo(RunParseResult.Name);
		for(const TPair<FString, FTextRange>& Pair : RunParseResult.MetaData)
		{
			RunInfo.MetaData.Add(Pair.Key, OriginalText.Mid(Pair.Value.BeginIndex, Pair.Value.EndIndex - Pair.Value.BeginIndex));
		}

		FTextRange ModelRange;
		ModelRange.BeginIndex = InOutModelText->Len();
		*InOutModelText += OriginalText.Mid(RunParseResult.ContentRange.BeginIndex, RunParseResult.ContentRange.EndIndex - RunParseResult.ContentRange.BeginIndex);
		ModelRange.EndIndex = InOutModelText->Len();

		return FSlateTextRun::Create(RunInfo, InOutModelText, TextStyles->CreateTextBlockStyle(RunInfo), ModelRange);
	}

private:

	FTextStyleDecoratorTest(FTextStyles* const InTextStyles)
		: TextStyles(InTextStyles)
	{
	}

	FTextStyles* TextStyles;
};

class SRichTextEditTest : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SRichTextEditTest ){}
	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 *
	 * @param InArgs   Declaration from which to construct the widget
	 */
	void Construct(const FArguments& InArgs)
	{
		// Define and add the "Roboto" font family
		TextStyles.AvailableFontFamilies.Emplace(MakeShareable(new FTextStyles::FFontFamily(
			LOCTEXT("RobotoFontFamily", "Roboto"), 
			TEXT("Roboto"), 
			FCoreStyle::GetDefaultFont()
			)));

		// Set some sensible defaults (these also match the default text style of "RichText.Editor.Text"
		ActiveFontFamily = TextStyles.AvailableFontFamilies[0];
		FontSize = 11;
		FontStyle = FTextStyles::EFontStyle::Regular;
		FontColor = FLinearColor::Black;

		// Define the example text
		// Both of our demo editors display this same text, but the text marshallers display it differently
		RichEditableText = FText::FromString(
			TEXT("<TextStyle FontFamily=\"Roboto\" FontSize=\"11\" FontStyle=\"Regular\" FontColor=\"(R=0.000000,G=0.000000,B=0.000000,A=1.000000)\">This is some regular text!</>")
			TEXT("\n")
			TEXT("<TextStyle FontFamily=\"Roboto\" FontSize=\"11\" FontStyle=\"Italic\" FontColor=\"(R=0.000000,G=0.000000,B=0.000000,A=1.000000)\">This is some italic text!</>")
			TEXT("\n")
			TEXT("<TextStyle FontFamily=\"Roboto\" FontSize=\"11\" FontStyle=\"Bold\" FontColor=\"(R=0.000000,G=0.000000,B=0.000000,A=1.000000)\">This is some bold text!</>")
			TEXT("\n")
			TEXT("<TextStyle FontFamily=\"Roboto\" FontSize=\"11\" FontStyle=\"BoldItalic\" FontColor=\"(R=0.000000,G=0.000000,B=0.000000,A=1.000000)\">This is some bold-italic text!</>")
			TEXT("\n")
			TEXT("<TextStyle FontFamily=\"Roboto\" FontSize=\"26\" FontStyle=\"BoldItalic\" FontColor=\"(R=1.000000,G=0.457315,B=0.164297,A=1.000000)\">This </><TextStyle FontFamily=\"Roboto\" FontSize=\"21\" FontStyle=\"Italic\" FontColor=\"(R=0.331120,G=0.348415,B=1.000000,A=1.000000)\">is </><TextStyle FontFamily=\"Roboto\" FontSize=\"21\" FontStyle=\"Bold\" FontColor=\"(R=1.000000,G=0.195252,B=0.137910,A=1.000000)\">some </><TextStyle FontFamily=\"Roboto\" FontSize=\"18\" FontStyle=\"BoldItalic\" FontColor=\"(R=1.000000,G=0.167594,B=0.566043,A=1.000000)\">crazy </><TextStyle FontFamily=\"Roboto\" FontSize=\"14\" FontStyle=\"Italic\" FontColor=\"(R=0.175489,G=0.695000,B=0.169137,A=1.000000)\">text!</>")
			TEXT("\n")
			TEXT("<TextStyle FontFamily=\"Roboto\" FontSize=\"11\" FontStyle=\"Regular\" FontColor=\"(R=0.000000,G=0.000000,B=0.000000,A=1.000000)\">This is a hyperlink: </><a id=\"browser\" href=\"https://www.unrealengine.com/\" style=\"RichText.Editor.Hyperlink\">Unreal Engine</><TextStyle FontFamily=\"Roboto\" FontSize=\"11\" FontStyle=\"Regular\" FontColor=\"(R=0.000000,G=0.000000,B=0.000000,A=1.000000)\"></>")
			);

		// The rich-text marshaller will be used with the rich-text editor...
		TSharedRef<FRichTextLayoutMarshaller> RichTextMarshaller = FRichTextLayoutMarshaller::Create(
			TArray<TSharedRef<ITextDecorator>>(), 
			&FTestStyle::Get()
			);

		// ... so we also need to add some decorators to handle the things we want to demo
		OnHyperlinkClicked = FSlateHyperlinkRun::FOnClick::CreateStatic(&RichTextHelper::OnBrowserLinkClicked, AsShared());
		RichTextMarshaller->AppendInlineDecorator(FHyperlinkDecorator::Create(TEXT("browser"), OnHyperlinkClicked));
		RichTextMarshaller->AppendInlineDecorator(FTextStyleDecoratorTest::Create(&TextStyles));

		// The syntax highlighter marshaller is self contained, so doesn't need any extra configuration
		SyntaxHighlighterMarshaller = FRichTextSyntaxHighlighterTextLayoutMarshaller::Create(
			FRichTextSyntaxHighlighterTextLayoutMarshaller::FSyntaxTextStyle()
			);

		this->ChildSlot
		[
			SNew(SScrollBox)
			+SScrollBox::Slot()
			[
				SNew(SBorder)
				.BorderImage(FTestStyle::Get().GetBrush("RichText.Background"))
				[
					SNew(SVerticalBox)

					// Demo title
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0)
					[
						SNew(SBorder)
						.BorderImage(FTestStyle::Get().GetBrush("RichText.Tagline.Background"))
						.Padding(0)
						[
							SNew(SRichTextBlock)
							.Text(LOCTEXT("RichTextEdit.Header01", "This WYSIWYG editor has been created to show the flexibility of Slate's <RichText.Tagline.TextHighlight>SMultiLineEditableText</>"))
							.TextStyle(FTestStyle::Get(), "RichText.Tagline.Text")
							.DecoratorStyleSet(&FTestStyle::Get())
							.AutoWrapText(true)
							.Justification(ETextJustify::Center)
							.Margin(FMargin(20))
						]
					]

					// Demo description
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0)
					[
						SNew(SRichTextBlock)
						.Text(LOCTEXT("RichTextEdit.HowItWorks", "<RichText.TextHighlight>SMultiLineEditableText</> utilizes the power of text marshallers to control how the source text is displayed, and the two editors below are demonstrating the <RichText.TextHighlight>FRichTextLayoutMarshaller</> and <RichText.TextHighlight>FRichTextSyntaxHighlighterTextLayoutMarshaller</> implementations.\n\nYou can use the toolbar to control the style of the text in the rich-text editor, or you can directly edit the source text (demonstrating live injection of style information) to update the rich-text view."))
						.TextStyle(FTestStyle::Get(), "RichText.Text")
						.DecoratorStyleSet(&FTestStyle::Get())
						.AutoWrapText(true)
						.Margin(FMargin(40, 20))
					]

					// Rich-text editor
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(FMargin(25, 0))
					[
						SNew(SBorder)
						.Padding(5.0f)
						.BorderImage(FCoreStyle::Get().GetBrush("BoxShadow"))
						[
							SNew(SBorder)
							.BorderImage(FTestStyle::Get().GetBrush("RichText.Interactive.Details.Background"))
							.Padding(FMargin(4))
							[
								// Rich-text editor toolbar
								SNew(SVerticalBox)

								+SVerticalBox::Slot()
								.AutoHeight()
								.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
								[
									SNew(SBorder)
									.BorderImage(FTestStyle::Get().GetBrush("RichText.RoundedBackground"))
									.Padding(FMargin(4))
									[
										SNew(SHorizontalBox)

										+SHorizontalBox::Slot()
										.AutoWidth()
										[
											SNew(SComboBox<TSharedPtr<FTextStyles::FFontFamily>>)
											.ComboBoxStyle(FTestStyle::Get(), "RichText.Toolbar.ComboBox")
											.OptionsSource(&TextStyles.AvailableFontFamilies)
											.OnSelectionChanged(this, &SRichTextEditTest::OnActiveFontFamilyChanged)
											.OnGenerateWidget(this, &SRichTextEditTest::GenerateFontFamilyComboEntry)
											.InitiallySelectedItem(ActiveFontFamily)
											[
												SNew(SBox)
												.Padding(FMargin(0.0f, 0.0f, 2.0f, 0.0f))
												[
													SNew(STextBlock)
													.Text(this, &SRichTextEditTest::GetActiveFontFamilyName)
												]
											]
										]

										+SHorizontalBox::Slot()
										.AutoWidth()
										[
											SNew(SBox)
											.MinDesiredWidth(24)
											[
												SNew(SNumericEntryBox<uint16>)
												.Value(this, &SRichTextEditTest::GetFontSize)
												.OnValueCommitted(this, &SRichTextEditTest::SetFontSize)
											]
										]

										+SHorizontalBox::Slot()
										.AutoWidth()
										.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
										[
											SNew(SCheckBox)
											.Style(FTestStyle::Get(), "RichText.Toolbar.ToggleButtonCheckbox")
											.IsChecked(this, &SRichTextEditTest::IsFontStyleBold)
											.OnCheckStateChanged(this, &SRichTextEditTest::OnFontStyleBoldChanged)
											[
												SNew(SBox)
												.WidthOverride(24)
												.HAlign(HAlign_Center)
												.VAlign(VAlign_Center)
												[
													SNew(STextBlock)
													.TextStyle(FTestStyle::Get(), "RichText.Toolbar.BoldText")
													.Text(LOCTEXT("BoldLabel", "B"))
												]
											]
										]

										+SHorizontalBox::Slot()
										.AutoWidth()
										[
											SNew(SCheckBox)
											.Style(FTestStyle::Get(), "RichText.Toolbar.ToggleButtonCheckbox")
											.IsChecked(this, &SRichTextEditTest::IsFontStyleItalic)
											.OnCheckStateChanged(this, &SRichTextEditTest::OnFontStyleItalicChanged)
											[
												SNew(SBox)
												.WidthOverride(24)
												.HAlign(HAlign_Center)
												.VAlign(VAlign_Center)
												[
													SNew(STextBlock)
													.TextStyle(FTestStyle::Get(), "RichText.Toolbar.ItalicText")
													.Text(LOCTEXT("ItalicLabel", "I"))
												]
											]
										]

										+SHorizontalBox::Slot()
										.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
										.AutoWidth()
										[
											SNew(SButton)
											.ButtonStyle(FTestStyle::Get(), "RichText.Toolbar.Button")
											.OnClicked(this, &SRichTextEditTest::OpenFontColorPicker)
											[
												SNew(SOverlay)

												+SOverlay::Slot()
												.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
												.HAlign(HAlign_Center)
												.VAlign(VAlign_Bottom)
												[
													SNew(STextBlock)
													.TextStyle(FTestStyle::Get(), "RichText.Toolbar.BoldText")
													.Text(LOCTEXT("ColorLabel", "A"))
												]

												+SOverlay::Slot()
												.HAlign(HAlign_Center)
												.VAlign(VAlign_Bottom)
												[
													SNew(SColorBlock)
													.Color(this, &SRichTextEditTest::GetFontColor)
													.Size(FVector2D(20.0f, 6.0f))
												]
											]
										]

										+SHorizontalBox::Slot()
										.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
										.AutoWidth()
										[
											SAssignNew(HyperlinkComboButton, SComboButton)
											.ComboButtonStyle(FTestStyle::Get(), "RichText.Toolbar.ComboButton")
											.HasDownArrow(false)
											.OnComboBoxOpened(this, &SRichTextEditTest::HandleHyperlinkComboOpened)
											.ButtonContent()
											[
												SNew(SBox)
												.WidthOverride(20)
												.HAlign(HAlign_Center)
												.VAlign(VAlign_Center)
												[
													SNew(SImage)
													.Image(FTestStyle::Get().GetBrush("RichText.Toolbar.HyperlinkImage"))
												]
											]
											.MenuContent()
											[
												SNew(SGridPanel)
												.FillColumn(1, 1.0f)

												+SGridPanel::Slot(0, 0)
												.HAlign(HAlign_Right)
												.Padding(FMargin(2.0f))
												[
													SNew(STextBlock)
													.TextStyle(FTestStyle::Get(), "RichText.Toolbar.Text")
													.Text(LOCTEXT("HyperlinkNameLabel", "Name:"))
												]
												+SGridPanel::Slot(1, 0)
												.Padding(FMargin(2.0f))
												[
													SNew(SBox)
													.WidthOverride(300)
													[
														SAssignNew(HyperlinkNameTextBox, SEditableTextBox)
													]
												]

												+SGridPanel::Slot(0, 1)
												.HAlign(HAlign_Right)
												.Padding(FMargin(2.0f))
												[
													SNew(STextBlock)
													.TextStyle(FTestStyle::Get(), "RichText.Toolbar.Text")
													.Text(LOCTEXT("HyperlinkURLLabel", "URL:"))
												]
												+SGridPanel::Slot(1, 1)
												.Padding(FMargin(2.0f))
												[
													SNew(SBox)
													.WidthOverride(300)
													[
														SAssignNew(HyperlinkURLTextBox, SEditableTextBox)
													]
												]

												+SGridPanel::Slot(0, 2)
												.HAlign(HAlign_Right)
												.Padding(FMargin(2.0f))
												.ColumnSpan(2)
												[
													SNew(SButton)
													.ButtonStyle(FTestStyle::Get(), "RichText.Toolbar.Button")
													.OnClicked(this, &SRichTextEditTest::HandleInsertHyperlinkClicked)
													[
														SNew(STextBlock)
														.TextStyle(FTestStyle::Get(), "RichText.Toolbar.Text")
														.Text(LOCTEXT("HyperlinkInsertLabel", "Insert Hyperlink"))
													]
												]
											]
										]
									]
								]

								+SVerticalBox::Slot()
								.AutoHeight()
								.Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
								[
									SAssignNew(RichEditableTextBox, SMultiLineEditableTextBox)
									.Font(FTestStyle::Get().GetWidgetStyle<FTextBlockStyle>("RichText.Editor.Text").Font)
									.Text(this, &SRichTextEditTest::GetRichEditableText)
									.OnTextChanged(this, &SRichTextEditTest::HandleRichEditableTextChanged)
									.OnTextCommitted(this, &SRichTextEditTest::HandleRichEditableTextCommitted)
									.OnCursorMoved(this, &SRichTextEditTest::HandleRichEditableTextCursorMoved)
									.Marshaller(RichTextMarshaller)
									.ClearTextSelectionOnFocusLoss(false)
									.AutoWrapText(true)
									.Margin(4)
									.LineHeightPercentage(1.1f)
								]
							]
						]
					]

					// Source text editor
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(FMargin(25, 0, 25, 25))
					[
						SNew(SBorder)
						.Padding(5.0f)
						.BorderImage(FCoreStyle::Get().GetBrush("BoxShadow"))
						[
							SNew(SBorder)
							.BorderImage(FTestStyle::Get().GetBrush("RichText.Interactive.Details.Background"))
							.Padding(FMargin(4))
							[
								SNew(SVerticalBox)

								+SVerticalBox::Slot()
								.AutoHeight()
								.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
								[
									SNew(SMultiLineEditableTextBox)
									.Text(this, &SRichTextEditTest::GetRichEditableText)
									.OnTextChanged(this, &SRichTextEditTest::HandleRichEditableTextChanged)
									.OnTextCommitted(this, &SRichTextEditTest::HandleRichEditableTextCommitted)
									.Marshaller(SyntaxHighlighterMarshaller)
									.AutoWrapText(true)
									.Margin(4)
									.LineHeightPercentage(1.1f)
								]

								+SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(SBorder)
									.BorderImage(FTestStyle::Get().GetBrush("RichText.RoundedBackground"))
									.Padding(FMargin(4))
									.HAlign(HAlign_Right)
									[
										SNew(SCheckBox)
										.Style(FTestStyle::Get(), "RichText.Toolbar.Checkbox")
										.IsChecked(this, &SRichTextEditTest::IsEnableSyntaxHighlightingChecked)
										.OnCheckStateChanged(this, &SRichTextEditTest::OnEnableSyntaxHighlightingChanged)
										[
											SNew(STextBlock)
											.TextStyle(FTestStyle::Get(), "RichText.Toolbar.Text")
											.Text(LOCTEXT("EnableSyntaxHighlightingLabel", "Enable Syntax Highlighting"))
										]
									]
								]
							]
						]
					]
				]
			]
		];
	}

protected:
	
	FText GetRichEditableText() const
	{
		return RichEditableText;
	}

	void HandleRichEditableTextChanged(const FText& Text)
	{
		RichEditableText = Text;
	}

	void HandleRichEditableTextCommitted(const FText& Text, ETextCommit::Type Type)
	{
		RichEditableText = Text;
	}

	void HandleRichEditableTextCursorMoved(const FTextLocation& NewCursorPosition )
	{
		// We can use GetRunUnderCursor to query the style of the text under the cursor
		// We can then use this to update the toolbar
		TSharedPtr<const IRun> Run = RichEditableTextBox->GetRunUnderCursor();
		if(Run.IsValid() && Run->GetRunInfo().Name == TEXT("TextStyle"))
		{
			TextStyles.ExplodeRunInfo(Run->GetRunInfo(), ActiveFontFamily, FontSize, FontStyle, FontColor);
		}
	}

	FText GetActiveFontFamilyName() const
	{
		return ActiveFontFamily->DisplayName;
	}

	void OnActiveFontFamilyChanged(TSharedPtr<FTextStyles::FFontFamily> NewValue, ESelectInfo::Type)
	{
		ActiveFontFamily = NewValue;
		StyleSelectedText();
	}

	TSharedRef<SWidget> GenerateFontFamilyComboEntry(TSharedPtr<FTextStyles::FFontFamily> SourceEntry)
	{
		return SNew(STextBlock).Text(SourceEntry->DisplayName);
	}

	TOptional<uint16> GetFontSize() const
	{
		return FontSize;
	}

	void SetFontSize(uint16 NewValue, ETextCommit::Type)
	{		
		FontSize = NewValue;
		StyleSelectedText();
	}

	ECheckBoxState IsFontStyleBold() const
	{
		return (FontStyle & FTextStyles::EFontStyle::Bold) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void OnFontStyleBoldChanged(ECheckBoxState InState)
	{
		if(InState == ECheckBoxState::Checked)
		{
			FontStyle |= FTextStyles::EFontStyle::Bold;
		}
		else
		{
			FontStyle &= ~FTextStyles::EFontStyle::Bold;
		}
		StyleSelectedText();
	}

	ECheckBoxState IsFontStyleItalic() const
	{
		return (FontStyle & FTextStyles::EFontStyle::Italic) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void OnFontStyleItalicChanged(ECheckBoxState InState)
	{
		if(InState == ECheckBoxState::Checked)
		{
			FontStyle |= FTextStyles::EFontStyle::Italic;
		}
		else
		{
			FontStyle &= ~FTextStyles::EFontStyle::Italic;
		}
		StyleSelectedText();
	}

	FLinearColor GetFontColor() const
	{
		return FontColor;
	}

	void SetFontColor(FLinearColor NewValue)
	{		
		FontColor = NewValue;
		StyleSelectedText();
	}

	FReply OpenFontColorPicker()
	{
		FColorPickerArgs PickerArgs;
		PickerArgs.bOnlyRefreshOnMouseUp = true;
		PickerArgs.ParentWidget = AsShared();
		PickerArgs.bUseAlpha = false;
		PickerArgs.bOnlyRefreshOnOk = false;
		PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &SRichTextEditTest::SetFontColor);
		PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP(this, &SRichTextEditTest::SetFontColor);
		PickerArgs.InitialColor = FontColor;

		OpenColorPicker(PickerArgs);

		return FReply::Handled();
	}

	void StyleSelectedText()
	{
		// Apply the current style to the selected text
		// If no text is selected, then a new (empty) run will be inserted with the appropriate style
		const FRunInfo RunInfo = TextStyles.CreateRunInfo(ActiveFontFamily, FontSize, FontStyle, FontColor);
		const FTextBlockStyle TextBlockStyle = TextStyles.CreateTextBlockStyle(ActiveFontFamily, FontSize, FontStyle, FontColor);
		RichEditableTextBox->ApplyToSelection(RunInfo, TextBlockStyle);
		FSlateApplication::Get().SetKeyboardFocus(RichEditableTextBox, EFocusCause::SetDirectly);
	}

	void HandleHyperlinkComboOpened()
	{
		// Read any currently selected text, and use this as the default name of the hyperlink
		FString SelectedText = RichEditableTextBox->GetSelectedText().ToString();
		for(int32 SelectedTextIndex = 0; SelectedTextIndex < SelectedText.Len(); ++SelectedTextIndex)
		{
			if(FChar::IsLinebreak(SelectedText[SelectedTextIndex]))
			{
				SelectedText.LeftInline(SelectedTextIndex, EAllowShrinking::No);
				break;
			}
		}
		HyperlinkNameTextBox->SetText(FText::FromString(SelectedText));

		// We can use GetRunUnderCursor to query whether the cursor is currently over a hyperlink
		// If it is, we can use that as the default URL for the hyperlink
		TSharedPtr<const IRun> Run = RichEditableTextBox->GetRunUnderCursor();
		if(Run.IsValid() && Run->GetRunInfo().Name == TEXT("a"))
		{
			const FString* const URLUnderCursor = Run->GetRunInfo().MetaData.Find(TEXT("href"));
			HyperlinkURLTextBox->SetText((URLUnderCursor) ? FText::FromString(*URLUnderCursor) : FText());
		}
		else
		{
			HyperlinkURLTextBox->SetText(FText());
		}
	}

	FReply HandleInsertHyperlinkClicked()
	{
		HyperlinkComboButton->SetIsOpen(false);

		const FText& Name = HyperlinkNameTextBox->GetText();
		const FText& URL = HyperlinkURLTextBox->GetText();

		// Create the correct meta-information for this run, so that valid source rich-text formatting can be generated for it
		FRunInfo RunInfo(TEXT("a"));
		RunInfo.MetaData.Add(TEXT("id"), TEXT("browser"));
		RunInfo.MetaData.Add(TEXT("href"), URL.ToString());
		RunInfo.MetaData.Add(TEXT("style"), TEXT("RichText.Editor.Hyperlink"));

		// Create the new run, and then insert it at the cursor position
		TSharedRef<FSlateHyperlinkRun> HyperlinkRun = FSlateHyperlinkRun::Create(
			RunInfo, 
			MakeShareable(new FString(Name.ToString())), 
			FTestStyle::Get().GetWidgetStyle<FHyperlinkStyle>(FName("RichText.Editor.Hyperlink")), 
			OnHyperlinkClicked,
			FSlateHyperlinkRun::FOnGenerateTooltip(),
			FSlateHyperlinkRun::FOnGetTooltipText()
			);
		RichEditableTextBox->InsertRunAtCursor(HyperlinkRun);

		return FReply::Handled();
	}

	ECheckBoxState IsEnableSyntaxHighlightingChecked() const
	{
		return (SyntaxHighlighterMarshaller->IsSyntaxHighlightingEnabled()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void OnEnableSyntaxHighlightingChanged(ECheckBoxState InState)
	{
		SyntaxHighlighterMarshaller->EnableSyntaxHighlighting(InState == ECheckBoxState::Checked);
	}

protected:

	FText RichEditableText;
	TSharedPtr<SMultiLineEditableTextBox> RichEditableTextBox;

	TSharedPtr<FSyntaxHighlighterTextLayoutMarshaller> SyntaxHighlighterMarshaller;

	FSlateHyperlinkRun::FOnClick OnHyperlinkClicked;
	TSharedPtr<SComboButton> HyperlinkComboButton;
	TSharedPtr<SEditableTextBox> HyperlinkNameTextBox;
	TSharedPtr<SEditableTextBox> HyperlinkURLTextBox;

	FTextStyles TextStyles;

	TSharedPtr<FTextStyles::FFontFamily> ActiveFontFamily;
	uint16 FontSize;
	FTextStyles::EFontStyle::Flags FontStyle;
	FLinearColor FontColor;
};

#endif //WITH_FANCY_TEXT

/** Demonstrates the brokenness of our current approach to trading smoothness for sharpness. */
 class SLayoutRoundingTest : public SCompoundWidget
 {
	SLATE_BEGIN_ARGS(SLayoutRoundingTest)
	{}
	SLATE_END_ARGS()

	static TSharedRef<SWidget> MakeRow(int32 NumWidgets)
	{
		TSharedRef<SHorizontalBox> HBox = SNew(SHorizontalBox);
		for (int ColIndex=0; ColIndex < NumWidgets; ++ColIndex)
		{
			HBox->AddSlot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity(FLinearColor(1,1,1,0.5f))
				.Image( FCoreStyle::Get().GetBrush("GenericWhiteBox") )
			];
		}
		return HBox;
	}

	void Construct( const FArguments& InArgs )
	{
		TSharedRef<SVerticalBox> VBox = SNew(SVerticalBox);

		for (int RowIndex=0; RowIndex < 15; ++RowIndex)
		{
			VBox->AddSlot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				MakeRow(15)
			];
		}

		this->ChildSlot
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.17)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.FillHeight(1.17)
				+SVerticalBox::Slot()
				.FillHeight(1)
				[
					VBox
				]
			]
		];			
	}
 };


/**
 * A list of commands for the multibox test
 */
class FMultiBoxTestCommandList : public TCommands<FMultiBoxTestCommandList>
{
public:
	FMultiBoxTestCommandList()
		: TCommands<FMultiBoxTestCommandList>( "MultiBoxTest", LOCTEXT("MultiboxTest", "Multibox Test"), NAME_None, FTestStyle::Get().GetStyleSetName() )
	{

	}
public:
	
	TSharedPtr<FUICommandInfo> FirstCommandInfo;

	TSharedPtr<FUICommandInfo> SecondCommandInfo;

	TSharedPtr<FUICommandInfo> ThirdCommandInfo;

	TSharedPtr<FUICommandInfo> FourthCommandInfo;


	TSharedPtr<FUICommandInfo> FifthCommandInfo;

	TSharedPtr<FUICommandInfo> SixthCommandInfo;

	TSharedPtr<FUICommandInfo> SeventhCommandInfo;

	TSharedPtr<FUICommandInfo> EighthCommandInfo;


public:
	void RegisterCommands()
	{
		UI_COMMAND( FirstCommandInfo, "First Test", "This is the first test menu item", EUserInterfaceActionType::ToggleButton, FInputChord() );
		UI_COMMAND( SecondCommandInfo, "Second Test", "This is the second test menu item. Shows a keybinding", EUserInterfaceActionType::ToggleButton, FInputChord( EModifierKey::Shift, EKeys::A ) );

		UI_COMMAND( ThirdCommandInfo, "Third Test", "This is the thrid test menu item", EUserInterfaceActionType::ToggleButton, FInputChord() );
		UI_COMMAND( FourthCommandInfo, "Fourth Test", "This is the fourth test menu item", EUserInterfaceActionType::ToggleButton, FInputChord() );


		UI_COMMAND( FifthCommandInfo, "Fifth Test", "This is the fifth test menu item", EUserInterfaceActionType::ToggleButton, FInputChord() );
		UI_COMMAND( SixthCommandInfo, "Sixth Test", "This is the sixth test menu item. Shows a keybinding", EUserInterfaceActionType::ToggleButton, FInputChord() );

		UI_COMMAND( SeventhCommandInfo, "Seventh Test", "This is the seventh test menu item", EUserInterfaceActionType::ToggleButton, FInputChord() );
		UI_COMMAND( EighthCommandInfo, "Eighth Test", "This is the eighth test menu item", EUserInterfaceActionType::ToggleButton, FInputChord() );
	}
};



class FMenus
{
public:
	static void FillMenu1Entries( FMenuBuilder& MenuBuilder )
	{
		MenuBuilder.BeginSection("Menu1Entries");
		{
			MenuBuilder.AddMenuEntry( FMultiBoxTestCommandList::Get().FirstCommandInfo );
			MenuBuilder.AddMenuEntry( FMultiBoxTestCommandList::Get().SecondCommandInfo );
			MenuBuilder.AddMenuEntry( FMultiBoxTestCommandList::Get().ThirdCommandInfo );
			MenuBuilder.AddMenuEntry( FMultiBoxTestCommandList::Get().FourthCommandInfo );
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("Menu1Entries2");
		{
			MenuBuilder.AddMenuEntry( FMultiBoxTestCommandList::Get().FifthCommandInfo );
			MenuBuilder.AddMenuEntry( FMultiBoxTestCommandList::Get().SixthCommandInfo );
			MenuBuilder.AddMenuEntry( FMultiBoxTestCommandList::Get().SeventhCommandInfo );
			MenuBuilder.AddMenuEntry( FMultiBoxTestCommandList::Get().EighthCommandInfo );
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("Menu1Entries3");
		{
			MenuBuilder.AddSubMenu( LOCTEXT("SubMenu", "Sub Menu"), LOCTEXT("OpensASubmenu", "Opens a submenu"), FNewMenuDelegate::CreateStatic( &FMenus::FillSubMenuEntries ) );
			MenuBuilder.AddSubMenu( LOCTEXT("SubMenu2IsALittleLonger", "Sub Menu 2 is a little longer"), LOCTEXT("OpensASubmenu", "Opens a submenu"), FNewMenuDelegate::CreateStatic( &FMenus::FillSubMenuEntries ) );
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("Menu1Entries4");
		{
			MenuBuilder.AddWidget(SNew(SVolumeControl), LOCTEXT("Volume", "Volume"));
		}
		MenuBuilder.EndSection();
	}

	static void FillMenu2Entries( FMenuBuilder& MenuBuilder )
	{
		MenuBuilder.AddEditableText( LOCTEXT("EditableItem", "Editable Item" ), LOCTEXT("EditableItem_ToolTip", "You can edit this item's text" ), FSlateIcon(), LOCTEXT("DefaultEditableText", "Edit Me!" )) ;

		MenuBuilder.AddMenuEntry( FMultiBoxTestCommandList::Get().FirstCommandInfo );

		MenuBuilder.AddMenuEntry( FMultiBoxTestCommandList::Get().SecondCommandInfo );

		MenuBuilder.AddSubMenu( LOCTEXT("SubMenu", "Sub Menu"), LOCTEXT("OpensASubmenu", "Opens a submenu"), FNewMenuDelegate::CreateStatic( &FMenus::FillSubMenuEntries ) );
		
		MenuBuilder.AddSubMenu( LOCTEXT("SubMenu2IsALittleLonger", "Sub Menu 2 is a little longer"), LOCTEXT("OpensASubmenu", "Opens a submenu"), FNewMenuDelegate::CreateStatic( &FMenus::FillSubMenuEntries ) );
	}

protected:
	static void FillSubMenuEntries( FMenuBuilder& MenuBuilder )
	{
		MenuBuilder.AddMenuEntry( FMultiBoxTestCommandList::Get().FirstCommandInfo );

		MenuBuilder.AddMenuEntry( FMultiBoxTestCommandList::Get().SecondCommandInfo );

		MenuBuilder.AddSubMenu( LOCTEXT("SubMenu", "Sub Menu"), LOCTEXT("OpensASubmenu", "Opens a submenu"), FNewMenuDelegate::CreateStatic( &FMenus::FillSubMenuEntries ) );

		MenuBuilder.AddSubMenu( LOCTEXT("SubMenu2IsALittleLonger", "Sub Menu 2 is a little longer"), LOCTEXT("OpensASubmenu", "Opens a submenu"), FNewMenuDelegate::CreateStatic( &FMenus::FillSubMenuEntries ) );
	}
};



class SMultiBoxTest
	: public SCompoundWidget
{

public:
	SLATE_BEGIN_ARGS( SMultiBoxTest ){}
	SLATE_END_ARGS()

	SMultiBoxTest()
		: CommandList( new FUICommandList() )
	{}

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct( const FArguments& InArgs )
	{
		ButtonStates.Reset();
		ButtonStates.AddZeroed( 8 );

		struct Local
		{
			static bool IsButtonEnabled( int32 ButtonIndex )
			{
				return true;
			}

			static void OnButtonPressed( int32 ButtonIndex )
			{
				const bool NewState = !ButtonStates[ ButtonIndex ];
				ButtonStates[ ButtonIndex ] = NewState;
			}

			static bool IsButtonChecked( int32 ButtonIndex )
			{
				return ButtonStates[ ButtonIndex ];
			}

		
		};

		FMultiBoxTestCommandList::Register();

		CommandList->MapAction( 
			FMultiBoxTestCommandList::Get().FirstCommandInfo,
			FExecuteAction::CreateStatic( &Local::OnButtonPressed, 0),
			FCanExecuteAction::CreateStatic( &Local::IsButtonEnabled, 0 ),
			FIsActionChecked::CreateStatic( &Local::IsButtonChecked, 0 )
		);

		CommandList->MapAction( 
			FMultiBoxTestCommandList::Get().SecondCommandInfo,
			FExecuteAction::CreateStatic( &Local::OnButtonPressed, 1 ),
			FCanExecuteAction::CreateStatic( &Local::IsButtonEnabled, 1 ),
			FIsActionChecked::CreateStatic( &Local::IsButtonChecked, 1 )
			);

		CommandList->MapAction( 
			FMultiBoxTestCommandList::Get().ThirdCommandInfo,
			FExecuteAction::CreateStatic( &Local::OnButtonPressed, 2 ),
			FCanExecuteAction::CreateStatic( &Local::IsButtonEnabled, 2 ),
			FIsActionChecked::CreateStatic( &Local::IsButtonChecked, 2 )
			);

		CommandList->MapAction( 
			FMultiBoxTestCommandList::Get().FourthCommandInfo,
			FExecuteAction::CreateStatic( &Local::OnButtonPressed, 3 ),
			FCanExecuteAction::CreateStatic( &Local::IsButtonEnabled, 3 ),
			FIsActionChecked::CreateStatic( &Local::IsButtonChecked, 3 )
			);


		CommandList->MapAction( 
			FMultiBoxTestCommandList::Get().FifthCommandInfo,
			FExecuteAction::CreateStatic( &Local::OnButtonPressed, 4 ),
			FCanExecuteAction::CreateStatic( &Local::IsButtonEnabled, 4 ),
			FIsActionChecked::CreateStatic( &Local::IsButtonChecked, 4 )
			);

		CommandList->MapAction( 
			FMultiBoxTestCommandList::Get().SixthCommandInfo,
			FExecuteAction::CreateStatic( &Local::OnButtonPressed, 5 ),
			FCanExecuteAction::CreateStatic( &Local::IsButtonEnabled, 5 ),
			FIsActionChecked::CreateStatic( &Local::IsButtonChecked, 5 )
			);

		CommandList->MapAction( 
			FMultiBoxTestCommandList::Get().SeventhCommandInfo,
			FExecuteAction::CreateStatic( &Local::OnButtonPressed, 6 ),
			FCanExecuteAction::CreateStatic( &Local::IsButtonEnabled, 6 ),
			FIsActionChecked::CreateStatic( &Local::IsButtonChecked, 6 )
			);

		CommandList->MapAction( 
			FMultiBoxTestCommandList::Get().EighthCommandInfo,
			FExecuteAction::CreateStatic( &Local::OnButtonPressed, 7 ),
			FCanExecuteAction::CreateStatic( &Local::IsButtonEnabled, 7 ),
			FIsActionChecked::CreateStatic( &Local::IsButtonChecked, 7 )
			);

		FMenuBarBuilder MenuBarBuilder( CommandList );
		{
			MenuBarBuilder.AddPullDownMenu( LOCTEXT("Menu1", "Menu 1"), LOCTEXT("OpensMenu1", "Opens Menu 1"), FNewMenuDelegate::CreateStatic( &FMenus::FillMenu1Entries ) );
			
			MenuBarBuilder.AddPullDownMenu( LOCTEXT("Menu2", "Menu 2"), LOCTEXT("OpensMenu2", "Opens Menu 2"), FNewMenuDelegate::CreateStatic( &FMenus::FillMenu2Entries ) );
		}


		this->ChildSlot
		[
			SNew( SVerticalBox )
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				MenuBarBuilder.MakeWidget()
			]
		];
	}

protected:
	TSharedRef< FUICommandList > CommandList;
	static TArray<bool> ButtonStates;
};

TArray<bool> SMultiBoxTest::ButtonStates;



class SAnimTest : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SAnimTest ){}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs )
	{
		this->ChildSlot
			. HAlign( HAlign_Fill )
			. VAlign( VAlign_Fill)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding( 3 )
			[

				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
						. Text( LOCTEXT("AnimTestDurationLabel", "Duration: ") )
				]
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0,0,5,0)
				[
					SNew(SSpinBox<float>)
						. MinValue(0.0f)
						. MaxValue(2.0f)
						. Delta(0.01f)
						. Value( this, &SAnimTest::GetAnimTime )
						. OnValueChanged( this, &SAnimTest::AnimTime_OnChanged )
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
						. Text( LOCTEXT("AnimTestPlayButtonLabel", "Play Animation") )
						. OnClicked( this, &SAnimTest::PlayAnimation_OnClicked )
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					. Text( LOCTEXT("AnimTestReverseButtonLabel", "Reverse") )
					. OnClicked( this, &SAnimTest::Reverse_OnClicked )
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					. Text( LOCTEXT("AnimTestPlayReverseButtonLabel", "PlayReverse") )
					. OnClicked( this, &SAnimTest::PlayReverse_OnClicked )
				]
			]
			+ SVerticalBox::Slot()
				. FillHeight(1)
			[
				SNew(SBorder)
				. ContentScale( this, &SAnimTest::GetContentScale )
				. HAlign(HAlign_Center)
				. VAlign(VAlign_Center)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
						.AutoHeight()
						. HAlign(HAlign_Fill)
						. VAlign(VAlign_Top)
					[
						// TITLE AREA
						SNew(SBorder)
							. Cursor( EMouseCursor::CardinalCross )
							. Padding( 3 )
							. HAlign(HAlign_Center)
						[
							SNew(STextBlock)
								. Text( LOCTEXT("AnimTestLabel", "Animation Testing") )
								. ColorAndOpacity( this, &SAnimTest::GetContentColor )
						]
					]
					+ SVerticalBox::Slot()
						. FillHeight(1)
						. HAlign(HAlign_Fill)
						. VAlign(VAlign_Fill)
					[
						// NODE CONTENT AREA
						SNew(SBorder)
							. HAlign(HAlign_Fill)
							. VAlign(VAlign_Fill)
							. Padding( FMargin(3.0f) )
							. ColorAndOpacity( this, &SAnimTest::GetContentColorAsLinearColor )
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
								.AutoWidth()
								. HAlign(HAlign_Left)
							[
								// LEFT
								SNew(SVerticalBox)
								+ SVerticalBox::Slot()
									.AutoHeight()
									. HAlign(HAlign_Center)
								[
									SNew(SImage)
										. Image( FCoreStyle::Get().GetBrush( TEXT("AppIcon") ) )
								]
								+ SVerticalBox::Slot()
									.AutoHeight()
								[
									SNew(SButton)
										. Text( LOCTEXT("ButtonTestLabel", "Button Test") )
								]
								+ SVerticalBox::Slot()
									.AutoHeight()
								[
									SNew(STextBlock)
										. Text( LOCTEXT("GenericTextItemTestLabel", "Generic Text Item") )
								]
								+ SVerticalBox::Slot()
									.AutoHeight()
								[
									SNew(SButton)
									[
										SNew(SHorizontalBox)
										+ SHorizontalBox::Slot()
										.AutoWidth()
										[
											SNew(SImage) .Image( FCoreStyle::Get().GetBrush( TEXT("AppIcon") ) )
										]
										+ SHorizontalBox::Slot()
										.AutoWidth()
										[
											SNew(STextBlock) .Text( LOCTEXT("ButtonTextLabel", "Button with content") )
										]
									]
								
								]
							]
							+ SHorizontalBox::Slot()
								. FillWidth(1)
								. Padding(5)
							[
								// MIDDLE
								SNew(SSpacer)
							]
							+ SHorizontalBox::Slot()
								.AutoWidth()
								. HAlign(HAlign_Right)
							[
								// RIGHT
								SNew(SImage) . Image( FTestStyle::Get().GetBrush( TEXT("GammaReference") ) )
							]
						]
					]
				]
			]
		];

		// Set the default Animation duration.
		AnimTime_OnChanged(0.15f);
	}

	FReply PlayAnimation_OnClicked()
	{
		SpawnAnimation.Play(this->AsShared());

		return FReply::Handled();
	}

	FReply Reverse_OnClicked()
	{
		SpawnAnimation.Reverse();

		return FReply::Handled();
	}

	FReply PlayReverse_OnClicked()
	{
		SpawnAnimation.PlayReverse(this->AsShared());

		return FReply::Handled();
	}

	float GetAnimTime() const 
	{
		return AnimTime;
	}

	void AnimTime_OnChanged( float InNewValue )
	{
		AnimTime = InNewValue;
		
		// Create the animation
		SpawnAnimation = FCurveSequence();
		{
			ZoomCurve = SpawnAnimation.AddCurve(0, AnimTime);
			FadeCurve = SpawnAnimation.AddCurve(AnimTime, AnimTime);
		}
	}

	FSlateColor GetContentColor() const
	{
		return GetContentColorAsLinearColor();
	}

	FLinearColor GetContentColorAsLinearColor() const
	{
		return FMath::Lerp( FLinearColor(1,1,1,0), FLinearColor(1,1,1,1), FadeCurve.GetLerp() );
	}

	FVector2D GetContentScale() const
	{
		const float ZoomValue = ZoomCurve.GetLerp();
		return FVector2D(ZoomValue, ZoomValue);
	}

	float AnimTime;

	FCurveSequence SpawnAnimation;
	FCurveHandle ZoomCurve;
	FCurveHandle FadeCurve;

};

class SFxTest : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( SFxTest )
	{}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs )
	{
		RenderScale = 1.0f;
		RenderScaleOrigin = FVector2D(0.5f, 0.5f);
		LayoutScale = 1.0f;
		VisualOffset = FVector2D::ZeroVector;
		FxWidgetIgnoreClippingState = ECheckBoxState::Checked;


		this->ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot() .AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				[
					SNew(SGridPanel)
					.FillColumn(1, 1.0f)
					+SGridPanel::Slot(0,0) .HAlign(HAlign_Right)
					[
						SNew(STextBlock)
						.Text( LOCTEXT("FxTextVisualScaleLabel", "Visual Scale:") )
					]
					+SGridPanel::Slot(1,0)  .Padding(2)
					[
						SNew(SSpinBox<float>)
						.MinValue(0.1f) .MaxValue(20.0f)
						.Value(this, &SFxTest::GetRenderScale)
						.OnValueChanged(this, &SFxTest::OnRenderScaleChanged)
					]
					+SGridPanel::Slot(0,1) .HAlign(HAlign_Right)
					[
						SNew(STextBlock)
						.Text( LOCTEXT("FxTextVisualScaleOriginLabel", "Visual Scale Origin:") )
					]
					+SGridPanel::Slot(1,1)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()  .Padding(2)
						[
							SNew(SSpinBox<float>)
							.MinValue(0.0f) .MaxValue(1.0f)
							.Value(this, &SFxTest::GetRenderScaleOriginX)
							.OnValueChanged(this, &SFxTest::OnRenderScaleChangedX)
						]
						+SHorizontalBox::Slot()  .Padding(2)
						[
							SNew(SSpinBox<float>)
							.MinValue(0.0f) .MaxValue(1.0f)
							.Value(this, &SFxTest::GetRenderScaleOriginY)
							.OnValueChanged(this, &SFxTest::OnRenderScaleChangedY)
						]
					]
					+SGridPanel::Slot(0,2) .HAlign(HAlign_Right)
					[
						SNew(STextBlock)
						.Text( LOCTEXT("FxTextLayoutScaleLabel", "Layout Scale:") )
					]
					+SGridPanel::Slot(1,2)  .Padding(2)
					[
						SNew(SSpinBox<float>)
						.MinValue(0.1f) .MaxValue(20.0f)
						.Value(this, &SFxTest::GetLayoutScale)
						.OnValueChanged(this, &SFxTest::OnLayoutScaleChanged)
					]
					+SGridPanel::Slot(0,3) .HAlign(HAlign_Right)
					[
						SNew(STextBlock)
						.Text( LOCTEXT("FxTextVisualOffsetLabel", "Visual Offset:") )
					]
					+SGridPanel::Slot(1,3)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot() .Padding(2)
						[
							SNew(SSpinBox<float>)
							.MinValue(-10.0f) .MaxValue(10.0f)
							.Value(this, &SFxTest::GetVisualOffsetOriginX)
							.OnValueChanged(this, &SFxTest::OnVisualOffsetChangedX)
						]
						+SHorizontalBox::Slot() .Padding(2)
						[
							SNew(SSpinBox<float>)
							.MinValue(-10.0f) .MaxValue(10.0f)
							.Value(this, &SFxTest::GetVisualOffsetOriginY)
							.OnValueChanged(this, &SFxTest::OnVisualOffsetChangedY)
						]
					]
					+SGridPanel::Slot(0, 4) .Padding(2)
					[
						SNew(SCheckBox)
						.IsChecked( this, &SFxTest::GetFxWidgetIgnoreClippingState )
						.OnCheckStateChanged( this, &SFxTest::OnFxWidgetIgnoreClippingChanged )
						[
							SNew(STextBlock)
							.Text( LOCTEXT("IgnoreClipping", "Ignore Clipping") )
						]
					]
				]
				+SHorizontalBox::Slot()
				[
					SNew(SUniformGridPanel)
					+SUniformGridPanel::Slot(0,0)
					[
						SNew(SButton)
						[
							SNew(STextBlock) .Text(LOCTEXT("FxTextZoomFadeOutLabel", "Zoom Fade Out"))
						]
					]
					+SUniformGridPanel::Slot(1,0)
					[
						SNew(SButton)
						[
							SNew(STextBlock) .Text(LOCTEXT("FxTextFadeInFromLeftLabel", "Fade in From Left"))
						]
					]
					+SUniformGridPanel::Slot(1,1)
					[
						SNew(SButton)
						[
							SNew(STextBlock) .Text(LOCTEXT("FxTextFadeInFromRightLabel", "Fade in From Right"))
						]
					]
				]

			]
			+SVerticalBox::Slot() .AutoHeight() .HAlign(HAlign_Center) .Padding(20)
			[
				SNew(SBorder)
				[
					SNew(SFxWidget)
					.IgnoreClipping(this, &SFxTest::GetFxWidgetIgnoreClipping)
					.RenderScale( this, &SFxTest::GetRenderScale )
					.RenderScaleOrigin( this, &SFxTest::GetRenderScaleOrigin )
					.LayoutScale( this, &SFxTest::GetLayoutScale )
					.VisualOffset( this, &SFxTest::GetVisualOffset )
					[
						SNew(SBorder)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot() .AutoWidth()
							[
								SNew(SImage)
								.Image(FTestStyle::Get().GetBrush("UEIcon"))
							]
							+SHorizontalBox::Slot() .AutoWidth()
							[
								SNew(SButton)
								.Text( LOCTEXT("FxTextContentLabel", "Click Me!" ) )
							]
						]
					]
				]
			]
		];
	}

	bool GetFxWidgetIgnoreClipping() const { return FxWidgetIgnoreClippingState == ECheckBoxState::Checked; }
	ECheckBoxState GetFxWidgetIgnoreClippingState() const { return FxWidgetIgnoreClippingState; }
	void OnFxWidgetIgnoreClippingChanged( ECheckBoxState InValue ) { FxWidgetIgnoreClippingState = InValue; }
	ECheckBoxState FxWidgetIgnoreClippingState;

	float GetRenderScale() const { return RenderScale; }
	void OnRenderScaleChanged( float InValue ) { RenderScale = InValue; }
	float RenderScale;

	FVector2D GetRenderScaleOrigin() const { return RenderScaleOrigin; }
	float GetRenderScaleOriginX() const { return RenderScaleOrigin.X; }
	float GetRenderScaleOriginY() const { return RenderScaleOrigin.Y; }
	void OnRenderScaleChangedX( float InValue ) { RenderScaleOrigin.X = InValue; }
	void OnRenderScaleChangedY( float InValue ) { RenderScaleOrigin.Y = InValue; }
	FVector2D RenderScaleOrigin;
	
	float GetLayoutScale() const { return LayoutScale; }
	void OnLayoutScaleChanged( float InValue ) { LayoutScale = InValue; }
	float LayoutScale;

	FVector2D GetVisualOffset() const { return VisualOffset; }
	float GetVisualOffsetOriginX() const { return VisualOffset.X; }
	float GetVisualOffsetOriginY() const { return VisualOffset.Y; }
	void OnVisualOffsetChangedX( float InValue ) { VisualOffset.X = InValue; }
	void OnVisualOffsetChangedY( float InValue ) { VisualOffset.Y = InValue; }
	FVector2D VisualOffset;
};



class SDPIScalingTest : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDPIScalingTest)
	{}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs )
	{
		DPIScale = 1.0f;
		
		this->ChildSlot
		.Padding(10)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot() .AutoHeight() .Padding( 5 )
			[
				SNew(SSpinBox<float>)
				.Value( this, &SDPIScalingTest::GetDPIScale )
				.OnValueChanged( this, &SDPIScalingTest::SetDPIScale )
			]

			+SVerticalBox::Slot() .AutoHeight() .Padding( 5 )
			[
				SNew(SCheckBox)
				.IsChecked( this, &SDPIScalingTest::IsFillChecked )
				.OnCheckStateChanged( this, &SDPIScalingTest::OnFillChecked )
				[
					SNew(STextBlock)
					.Text( LOCTEXT("DpiScalingFillSpaceLabel", "Fill Space") )
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding( 5 )
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Expose( ScalerSlot )
			[

				SNew(SDPIScaler)
				.DPIScale( this, &SDPIScalingTest::GetDPIScale )
				[
					SNew(SBorder)
					.BorderImage( FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder") )
					.Padding(5)
					[
						SNew(STextBlock)
						.Text( LOCTEXT("DpiScalingTextLabel", "I am DPI scaled!") )
					]
				]
			]
		];
	}

	float GetDPIScale() const
	{
		return DPIScale;
	}

	void SetDPIScale( float InScale )
	{
		DPIScale = InScale;
	}

	ECheckBoxState IsFillChecked() const
	{
		const bool bIsFilling = (ScalerSlot->GetHorizontalAlignment() == HAlign_Fill);
		return (bIsFilling)
			? ECheckBoxState::Checked
			: ECheckBoxState::Unchecked;
	}

	void OnFillChecked(ECheckBoxState InValue)
	{
		ScalerSlot->SetHorizontalAlignment( (InValue == ECheckBoxState::Checked) ? HAlign_Fill : HAlign_Center );
		ScalerSlot->SetVerticalAlignment( (InValue == ECheckBoxState::Checked) ? VAlign_Fill : VAlign_Center );
	}

	float DPIScale;
	SVerticalBox::FSlot* ScalerSlot;
};

class SGlobalInvalidationTest : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SGlobalInvalidationTest)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		ChildSlot
		.Padding(10)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SButton)
				.Text_Lambda([]() {
					return FText::Format(LOCTEXT("ToggleInvalidationMode", "Toggle Slate Global Invalidation [{0}]"),
						GSlateEnableGlobalInvalidation ? LOCTEXT("Off", "Off") : LOCTEXT("On", "On"));
				})
				.OnClicked_Lambda([this]()
				{
					FSlateApplication::Get().ToggleGlobalInvalidation(!GSlateEnableGlobalInvalidation);
					return FReply::Handled();
				})
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			[
				SNew(SBox)
				.WidthOverride(200)
				.HeightOverride(150)
				[
					SNew(SScrollBox)

					+ SScrollBox::Slot()[SNew(SButton).Text(LOCTEXT("Button", "Button"))]
					+ SScrollBox::Slot()[SNew(SButton).Text(LOCTEXT("Button", "Button"))]
					+ SScrollBox::Slot()[SNew(SButton).Text(LOCTEXT("Button", "Button"))]
					+ SScrollBox::Slot()[SNew(SButton).Text(LOCTEXT("Button", "Button"))]
					+ SScrollBox::Slot()[SNew(SButton).Text(LOCTEXT("Button", "Button"))]
					+ SScrollBox::Slot()[SNew(SButton).Text(LOCTEXT("Button", "Button"))]
					+ SScrollBox::Slot()[SNew(SButton).Text(LOCTEXT("Button", "Button"))]
					+ SScrollBox::Slot()[SNew(SButton).Text(LOCTEXT("Button", "Button"))]
					+ SScrollBox::Slot()[SNew(SButton).Text(LOCTEXT("Button", "Button"))]
					+ SScrollBox::Slot()[SNew(SButton).Text(LOCTEXT("Button", "Button"))]
				]
			]
		];
	}
};

class SInvalidationTest : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SInvalidationTest)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		ChildSlot
		.Padding(10)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SButton)
				.Text(LOCTEXT("OpenSlateGlobalInvalidationTestWindow", "Open Slate Global Invalidation Test Window"))
				.OnClicked_Lambda([this]()
				{
					TSharedRef<SWindow> TestWindow = SNew(SWindow)
					.ClientSize(FVector2D(640,480))
					.AutoCenter(EAutoCenter::PrimaryWorkArea)
					[
						SNew(SGlobalInvalidationTest)
					];

					TestWindow->SetAllowFastUpdate(true);

					FSlateApplication::Get().AddWindow( TestWindow );
					return FReply::Handled();
				})
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Invalidation Panels", "INVALIDATION PANELS"))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(CachePanel1, SInvalidationPanel)
				.Visibility(EVisibility::Visible)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("StaticText", "This text is static and cached."))
					]
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SButton)
				.Text(LOCTEXT("InvalidateManually", "Manually Invalidate"))
				.OnClicked(this, &SInvalidationTest::ManuallyInvalidatePanel1)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SInvalidationPanel)
				.Visibility(EVisibility::Visible)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("StaticTextSupportInput", "Support Input"))
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBorder)
						.Padding(10)
						[
							SNew(SButton)
							.Text(LOCTEXT("CachedClickable", "I'm Cached - But Clickable"))
						]
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBorder)
						.Padding(10)
						[
							SNew(SButton)
							.ForceVolatile(true)
							.Text(LOCTEXT("VolatileClickable", "Volatile - But Clickable"))
						]
					]
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SInvalidationPanel)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("RelativeCacheMessage", "I cache relative transforms.  Moving the window has no effect on me, woo!"))
					]
				]
			]
		];
	}

private:
	FReply ManuallyInvalidatePanel1()
	{
		CachePanel1->InvalidateRootChildOrder();
		return FReply::Handled();
	}

private:
	TSharedPtr<SInvalidationPanel> CachePanel1;
};

class SGammaTest : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SGammaTest)
	{}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs)
	{
		FColor Orange(200, 80, 15);
		
		ChildSlot
		.Padding(10)
		[
			SNew(SVerticalBox)
			
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
			 
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SColorBlock)
					.Color(Orange)
				]
			 
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(FText::FromString(Orange.ToString()))
				]
			]
		];
	}
};

class SColorPickerTest : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SColorPickerTest)
		: _Color(FLinearColor(1, 1, 1, 0.5f))
		{}

		SLATE_ATTRIBUTE(FLinearColor, Color)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		Color = InArgs._Color;

		this->ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
						.Text( LOCTEXT("ColorPickerTest-EditColorLabel", "Edit Color") )
						.ContentPadding( 5 )
						.OnClicked( this, &SColorPickerTest::OpenColorPicker )
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
						.Text( LOCTEXT("ColorPickerTest-EditNoAlphaColorLabel", "Edit NoAlpha Color") )
						.ContentPadding( 5 )
						.OnClicked( this, &SColorPickerTest::OpenColorPickerNoAlpha )
				]
				+SHorizontalBox::Slot().AutoWidth() .Padding(5)
				[
					SAssignNew(OutputTextBlock, STextBlock)
				]
			]
		];

		UpdateColor(Color.Get());
	}

protected:
	FReply OpenColorPicker()
	{
		TSharedRef<SWindow> NewSlateWindow = FSlateApplication::Get().AddWindow(
			SNew(SWindow)
			.Title(LOCTEXT("ColorPickerTest-WindowTitle-StandardColor", "Standard Color"))
			.ClientSize(SColorPicker::DEFAULT_WINDOW_SIZE)
		);

		TSharedPtr<SColorPicker> ColorPicker = SNew(SColorPicker)
			.TargetColorAttribute(this, &SColorPickerTest::GetColor)
			.OnColorCommitted( this, &SColorPickerTest::UpdateColor )
			.ParentWindow(NewSlateWindow);

		NewSlateWindow->SetContent(ColorPicker.ToSharedRef());

		return FReply::Handled();
	}
	
	FReply OpenColorPickerNoAlpha()
	{
		TSharedRef<SWindow> NewSlateWindow = FSlateApplication::Get().AddWindow(
			SNew(SWindow)
			.Title(LOCTEXT("ColorPickerTest-WindowTitle-NoAlphaColor", "No Alpha Color"))
			.ClientSize(SColorPicker::DEFAULT_WINDOW_SIZE)
		);

		TSharedPtr<SColorPicker> ColorPicker = SNew(SColorPicker)
			.UseAlpha(false)
			.TargetColorAttribute(this, &SColorPickerTest::GetColor)
			.OnColorCommitted( this, &SColorPickerTest::UpdateColor )
			.ParentWindow(NewSlateWindow);

		NewSlateWindow->SetContent(ColorPicker.ToSharedRef());

		return FReply::Handled();
	}

	FLinearColor GetColor() const
	{
		return Color.Get();
	}

	void UpdateColor(FLinearColor NewColor)
	{
		Color.Set(NewColor);
		OutputTextBlock->SetText( FText::FromString(Color.Get().ToFColor(false).ToString()) );
	}
	
	TAttribute<FLinearColor> Color;

	TSharedPtr<STextBlock> OutputTextBlock;
};

class STestNotificationWidget : public SCompoundWidget, public INotificationWidget
{
public:

	SLATE_BEGIN_ARGS(STestNotificationWidget){}
	SLATE_END_ARGS()

		void Construct(const FArguments& InArgs)
	{
		ChildSlot
			[
				SNew(SBox)
				.Padding(15.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FTestStyle::Get().GetBrush("UEIcon"))
					]
					+ SHorizontalBox::Slot()
					.Padding(FMargin(15.0f, 0.0f, 0.0f, 0.0f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("TestingBigNotificationText", "Big notififcation text!"))
						.Font(FCoreStyle::GetDefaultFontStyle("Bold", 30))
					]
					+ SHorizontalBox::Slot()
					.Padding(FMargin(15.0f, 0.0f, 0.0f, 0.0f))
					[
						SNew(SButton)
						.Text(LOCTEXT("TestButtonInNotificaiton", "Button Test"))
					]
				]
			];
	}

	virtual void OnSetCompletionState(SNotificationItem::ECompletionState State) override
	{
	}

	virtual TSharedRef< SWidget > AsWidget() override
	{
		return SharedThis(this);
	}
};

class SNotificationListTest : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNotificationListTest){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		bUseLargeFont = true;
		bAddDummyButtons = false;
		bAddDummyCheckBox = false;
		bAddDummyHyperlink = false;

		DummyCheckBoxState = ECheckBoxState::Unchecked;

		this->ChildSlot
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot().AutoWidth()
					[
						SNew(SButton)
						.OnClicked(this, &SNotificationListTest::SpawnNotification1)
						.Text( LOCTEXT("NotificationListTest-SpawnNotification1Label", "Spawn Notification1" ) )
					]

					+SHorizontalBox::Slot().AutoWidth()
					[
						SNew(SButton)
						.OnClicked(this, &SNotificationListTest::SpawnNotification2)
						.Text( LOCTEXT("NotificationListTest-SpawnNotification2Label", "Spawn Notification2") )
					]

					+ SHorizontalBox::Slot().AutoWidth()
						[
							SNew(SButton)
							.OnClicked(this, &SNotificationListTest::SpawnCustomNotification)
							.Text(LOCTEXT("NotificationListTest-SpawnCustomNotificationLabel", "Spawn Custom Notification"))
						]

					+SHorizontalBox::Slot().AutoWidth()
					[
						SNew(SButton)
						.OnClicked(this, &SNotificationListTest::SpawnPendingNotification)
						.Text( LOCTEXT("NotificationListTest-SpawnPendingNotificationLabel", "Spawn Pending Notification") )
					]

					+SHorizontalBox::Slot().AutoWidth()
					[
						SNew(SButton)
						.OnClicked(this, &SNotificationListTest::FinishPendingNotification_Success)
						.Text( LOCTEXT("NotificationListTest-FinishPendingNotificationSuccessLabel", "Finish Pending Notification - Success") )
					]

					+SHorizontalBox::Slot().AutoWidth()
					[
						SNew(SButton)
						.OnClicked(this, &SNotificationListTest::FinishPendingNotification_Fail)
						.Text( LOCTEXT("NotificationListTest-FinishPendingNotificationFailLabel", "Finish Pending Notification - Fail") )
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SCheckBox)
					.IsChecked(this, &SNotificationListTest::IsUseLargeFontChecked)
					.OnCheckStateChanged(this, &SNotificationListTest::OnUseLargeFontCheckStateChanged)
					[
						SNew(STextBlock)
						.Text( LOCTEXT("NotificationListTest-UseLargeFont", "Use Large Font" ) )
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SCheckBox)
					.IsChecked(this, &SNotificationListTest::IsAddDummyButtonsChecked)
					.OnCheckStateChanged(this, &SNotificationListTest::OnAddDummyButtonsCheckStateChanged)
					[
						SNew(STextBlock)
						.Text( LOCTEXT("NotificationListTest-AddDummyButtons", "Add Dummy Buttons" ) )
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SCheckBox)
					.IsChecked(this, &SNotificationListTest::IsAddDummyCheckBoxChecked)
					.OnCheckStateChanged(this, &SNotificationListTest::OnAddDummyCheckBoxCheckStateChanged)
					[
						SNew(STextBlock)
						.Text( LOCTEXT("NotificationListTest-AddDummyCheckBox", "Add Dummy Check Box" ) )
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SCheckBox)
					.IsChecked(this, &SNotificationListTest::IsAddDummyHyperlinkChecked)
					.OnCheckStateChanged(this, &SNotificationListTest::OnAddDummyHyperlinkCheckStateChanged)
					[
						SNew(STextBlock)
						.Text( LOCTEXT("NotificationListTest-AddDummyHyperlink", "Add Dummy Hyperlink" ) )
					]
				]
			]

			+SOverlay::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			.Padding(15)
			[
				SAssignNew(NotificationListPtr, SNotificationList)
				.Visibility(EVisibility::SelfHitTestInvisible)
			]
		];
	}

protected:
	ECheckBoxState IsUseLargeFontChecked() const
	{
		return bUseLargeFont ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void OnUseLargeFontCheckStateChanged(ECheckBoxState NewState)
	{
		bUseLargeFont = (NewState == ECheckBoxState::Checked);
	}

	ECheckBoxState IsAddDummyButtonsChecked() const
	{
		return bAddDummyButtons ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void OnAddDummyButtonsCheckStateChanged(ECheckBoxState NewState)
	{
		bAddDummyButtons = (NewState == ECheckBoxState::Checked);
	}

	ECheckBoxState IsAddDummyCheckBoxChecked() const
	{
		return bAddDummyCheckBox ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void OnAddDummyCheckBoxCheckStateChanged(ECheckBoxState NewState)
	{
		bAddDummyCheckBox = (NewState == ECheckBoxState::Checked);
	}

	ECheckBoxState IsAddDummyHyperlinkChecked() const
	{
		return bAddDummyHyperlink ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void OnAddDummyHyperlinkCheckStateChanged(ECheckBoxState NewState)
	{
		bAddDummyHyperlink = (NewState == ECheckBoxState::Checked);
	}

	void SetNotificationInfoFlags(FNotificationInfo& Info)
	{
		// Dummy callback for buttons and hyperlinks
		struct Local
		{
			static void Dummy()
			{
			}
		};
		const FSimpleDelegate DummyDelegate = FSimpleDelegate::CreateStatic(&Local::Dummy);

		Info.bUseLargeFont = bUseLargeFont;

		if(bAddDummyButtons)
		{
			Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("NotificationListTest-DummyButton1Text", "Dummy Button 1"), FText(), DummyDelegate, SNotificationItem::CS_None));
			Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("NotificationListTest-DummyButton2Text", "Dummy Button 2"), FText(), DummyDelegate, SNotificationItem::CS_None));

			Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("NotificationListTest-DummyPendingButton1Text", "Dummy Pending Button 1"), FText(), DummyDelegate, SNotificationItem::CS_Pending));
			Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("NotificationListTest-DummyPendingButton2Text", "Dummy Pending Button 2"), FText(), DummyDelegate, SNotificationItem::CS_Pending));

			Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("NotificationListTest-DummySuccessButton1Text", "Dummy Success Button 1"), FText(), DummyDelegate, SNotificationItem::CS_Success));
			Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("NotificationListTest-DummySuccessButton2Text", "Dummy Success Button 2"), FText(), DummyDelegate, SNotificationItem::CS_Success));

			Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("NotificationListTest-DummyFailButton1Text", "Dummy Fail Button 1"), FText(), DummyDelegate, SNotificationItem::CS_Fail));
			Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("NotificationListTest-DummyFailButton2Text", "Dummy Fail Button 2"), FText(), DummyDelegate, SNotificationItem::CS_Fail));
		}

		if(bAddDummyCheckBox)
		{
			Info.CheckBoxState = TAttribute<ECheckBoxState>(this, &SNotificationListTest::GetDummyCheckBoxState);
			Info.CheckBoxStateChanged = FOnCheckStateChanged::CreateSP(this, &SNotificationListTest::OnDummyCheckBoxStateChanged);
			Info.CheckBoxText = LOCTEXT("NotificationListTest-DummyCheckBoxText", "Dummy Check Box");
		}

		if(bAddDummyHyperlink)
		{
			Info.Hyperlink = DummyDelegate;
			Info.HyperlinkText = LOCTEXT("NotificationListTest-DummyHyperlinkText", "Dummy Hyperlink");
		}
	}

	ECheckBoxState GetDummyCheckBoxState() const
	{
		return DummyCheckBoxState;
	}

	void OnDummyCheckBoxStateChanged(ECheckBoxState NewState)
	{
		DummyCheckBoxState = NewState;
	}

	FReply SpawnNotification1()
	{
		FNotificationInfo Info( LOCTEXT("TestNotification01", "A Notification" ));
		SetNotificationInfoFlags(Info);
		NotificationListPtr->AddNotification(Info);
		return FReply::Handled();
	}

	FReply SpawnNotification2()
	{
		FNotificationInfo Info( LOCTEXT("TestNotification02", "Another Notification" ));
		SetNotificationInfoFlags(Info);
		NotificationListPtr->AddNotification(Info);
		return FReply::Handled();
	}

	FReply SpawnCustomNotification()
	{
		FNotificationInfo Info(SNew(STestNotificationWidget));
		Info.bFireAndForget = true;
		Info.ExpireDuration = 3.0f;
		Info.FadeOutDuration = 3.0f;

		SetNotificationInfoFlags(Info);
		NotificationListPtr->AddNotification(Info);

		return FReply::Handled();
	}

	FReply SpawnPendingNotification()
	{
		if ( PendingProgressPtr.IsValid() )
		{
			PendingProgressPtr.Pin()->ExpireAndFadeout();
		}

		FNotificationInfo Info(LOCTEXT("TestNotificationInProgress", "Operation In Progress" ));
		SetNotificationInfoFlags(Info);
		Info.bFireAndForget = false;
		
		PendingProgressPtr = NotificationListPtr->AddNotification(Info);

		PendingProgressPtr.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
		return FReply::Handled();
	}

	FReply FinishPendingNotification_Success()
	{
		TSharedPtr<SNotificationItem> NotificationItem = PendingProgressPtr.Pin();

		if ( NotificationItem.IsValid() )
		{
			NotificationItem->SetText( LOCTEXT("TestNotificationSuccess", "Operation Successful!") );
			NotificationItem->SetCompletionState(SNotificationItem::CS_Success);
			NotificationItem->ExpireAndFadeout();

			PendingProgressPtr.Reset();
		}

		return FReply::Handled();
	}

	FReply FinishPendingNotification_Fail()
	{
		TSharedPtr<SNotificationItem> NotificationItem = PendingProgressPtr.Pin();

		if ( NotificationItem.IsValid() )
		{
			NotificationItem->SetText( LOCTEXT("TestNotificationFailure", "Operation Failed...") );
			NotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
			NotificationItem->ExpireAndFadeout();

			PendingProgressPtr.Reset();
		}

		return FReply::Handled();
	}

	/** Should we use the larger bolder font to display the message? */
	bool bUseLargeFont;

	/** Should we add dummy buttons to the notifications? */
	bool bAddDummyButtons;

	/** Should we add a dummy check box to the notifications? */
	bool bAddDummyCheckBox;

	/** Should we add a dummy hyperlink to the notifications? */
	bool bAddDummyHyperlink;

	/** The list of active system messages */
	TSharedPtr<SNotificationList> NotificationListPtr;
	/** The pending progress message */
	TWeakPtr<SNotificationItem> PendingProgressPtr;
	/** If showing a dummy check box, this maintains its state in the UI */
	ECheckBoxState DummyCheckBoxState;
};

class SGridPanelTest : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGridPanelTest)
	{}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs )	
	{
		this->ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1)
			[
				SNew(SGridPanel)
				.FillColumn(1, 1.0f)
				.FillColumn(2, 2.0f)
				+SGridPanel::Slot(0,0)
				[
					SNew(STextBlock)
					.Text( LOCTEXT("GridPanelTest-Label01", "There once was a man from Nantucket") )
				]
				+SGridPanel::Slot(0,1)
				[
					SNew(STextBlock)
					.Text( LOCTEXT("GridPanelTest-Label02", "who kept all his cash in a bucket.") )
				]
				+SGridPanel::Slot(0,2)
				[
					SNew(STextBlock)
					.Text( LOCTEXT("GridPanelTest-Label03", "But his daughter, named Nan,") )
				]
				+SGridPanel::Slot(0,3)
				[
					SNew(STextBlock)
					.Text( LOCTEXT("GridPanelTest-Label04", "Ran away with a man") )
				]
				+SGridPanel::Slot(0,4)
				[
					SNew(STextBlock)
					.Text( LOCTEXT("GridPanelTest-Label05", "And as for the bucket, Nantucket.") )
				]
				+SGridPanel::Slot(1,1)
				.ColumnSpan(2)
				[
					SNew(SBorder)
					[
						SNew(STextBlock).Text( LOCTEXT("GridPanelTest-Colspan1Label", "Colspan = 1") )
					]
				]
				+SGridPanel::Slot(1,2)
				[
					SNew(SBorder)
					[
						SNew(STextBlock).Text( LOCTEXT("GridPanelTest-Stretch1Label", "Stretch = 1"))
					]
				]
				+SGridPanel::Slot(2,3)
				[
					SNew(SBorder)
					[
						SNew(STextBlock).Text( LOCTEXT("GridPanelTest-Stretch2Label", "Stretch = 2"))
					]
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1)
			[
				SNew(SGridPanel)
				.FillRow(1, 1.0f)
				.FillRow(2, 2.0f)
				+SGridPanel::Slot(0,0)
				[
					SNew(STextBlock)
					.Text( LOCTEXT("GridPanelTest-Label01", "There once was a man from Nantucket") )
				]
				+SGridPanel::Slot(0,1)
				[
					SNew(STextBlock)
					.Text( LOCTEXT("GridPanelTest-Label02", "who kept all his cash in a bucket.") )
				]
				+SGridPanel::Slot(0,2)
				[
					SNew(STextBlock)
					.Text( LOCTEXT("GridPanelTest-Label03", "But his daughter, named Nan,") )
				]
				+SGridPanel::Slot(0,3)
				[
					SNew(STextBlock)
					.Text( LOCTEXT("GridPanelTest-Label04", "Ran away with a man") )
				]
				+SGridPanel::Slot(0,4)
				[
					SNew(STextBlock)
					.Text( LOCTEXT("GridPanelTest-Label05", "And as for the bucket, Nantucket.") )
				]
				+SGridPanel::Slot(1,1)
				.RowSpan(2)
				[
					SNew(SBorder)
					[
						SNew(STextBlock).Text( LOCTEXT("GridPanelTest-Rowspan2Label", "RowSpan = 2!"))
					]
				]
				+SGridPanel::Slot(2,1)
				[
					SNew(SBorder)
					[
						SNew(STextBlock).Text( LOCTEXT("GridPanelTest-Stretch1Label", "Stretch = 1"))
					]
				]
				+SGridPanel::Slot(3,2)
				[
					SNew(SBorder)
					[
						SNew(STextBlock).Text( LOCTEXT("GridPanelTest-Stretch2Label", "Stretch = 2"))
					]
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1)
			
		];
	}
};

class SResponsiveGridPanelTestWidget : public SUserWidget
{

public:

	SLATE_USER_ARGS(SResponsiveGridPanelTestWidget)
	{}

	SLATE_END_ARGS()

	virtual void Construct(const FArguments& InArgs) = 0;

};

class SResponsiveGridPanelTestWidgetImpl : public SResponsiveGridPanelTestWidget
{
public:
	virtual void Construct(const FArguments& InArgs) override;

private:
	TSharedRef<SWidget> ConstructBox(const FString& Text) const;
};

TSharedRef<SWidget> SResponsiveGridPanelTestWidgetImpl::ConstructBox(const FString& Text) const
{
	return
		SNew(SBorder)
		.Padding(1)
		.BorderImage(FTestStyle::Get().GetBrush("Gray"))
		[
			SNew(SBorder)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(10)
			.BorderImage(FTestStyle::Get().GetBrush("RichText.Tagline.Background"))
			[
				SNew(STextBlock).Text(FText::FromString(Text))
			]
		];
}

void SResponsiveGridPanelTestWidgetImpl::Construct(const FArguments& InArgs)
{
	TSharedPtr<SResponsiveGridPanel> GridPanel;

	SUserWidget::Construct(SUserWidget::FArguments()
	[
		SNew(SBorder)
		.BorderImage(FTestStyle::Get().GetBrush("RichText.Background"))
		.ForegroundColor(FLinearColor::Black)
		[
			SNew(SScrollBox)
			+ SScrollBox::Slot()
			[
				SNew(SBox)
				.Padding(FMargin(50, 10))
				[
					SNew(SBorder)
					.BorderImage(FTestStyle::Get().GetBrush("RichText.Tagline.DarkBackground"))
					.Padding(10.0f)
					[
						SAssignNew(GridPanel, SResponsiveGridPanel, 12)
						.ColumnGutter(5)
						.RowGutter(5)
					]
				]
			]

			+ SScrollBox::Slot()
			.Padding(0, 20, 0, 0)
			[
				SNew(SBox)
				.Padding(FMargin(50, 10))
				[
					SNew(SBorder)
					.BorderImage(FTestStyle::Get().GetBrush("RichText.Tagline.DarkBackground"))
					.Padding(10.0f)
					[
						SNew(SResponsiveGridPanel, 12)
						.ColumnGutter(5)
						.RowGutter(5)
						+ SResponsiveGridPanel::Slot(0)
						.ColumnSpan(SResponsiveGridSize::MediumDevice, 8)
						.ColumnSpan(SResponsiveGridSize::Mobile, 12)
						[
							ConstructBox(TEXT(".mobile-col-12 .desktop-col-8"))
						]
						+ SResponsiveGridPanel::Slot(0)
						.ColumnSpan(SResponsiveGridSize::MediumDevice, 4)
						.ColumnSpan(SResponsiveGridSize::Mobile, 6)
						[
							ConstructBox(TEXT(".mobile-col-6 .desktop-col-4"))
						]
						+ SResponsiveGridPanel::Slot(1)
						.ColumnSpan(SResponsiveGridSize::MediumDevice, 4)
						.ColumnSpan(SResponsiveGridSize::Mobile, 6)
						[
							ConstructBox(TEXT(".mobile-col-6 .desktop-col-4"))
						]
						+ SResponsiveGridPanel::Slot(1)
						.ColumnSpan(SResponsiveGridSize::MediumDevice, 4)
						.ColumnSpan(SResponsiveGridSize::Mobile, 6)
						[
							ConstructBox(TEXT(".mobile-col-6 .desktop-col-4"))
						]
						+ SResponsiveGridPanel::Slot(1)
						.ColumnSpan(SResponsiveGridSize::MediumDevice, 4)
						.ColumnSpan(SResponsiveGridSize::Mobile, 6)
						[
							ConstructBox(TEXT(".mobile-col-6 .desktop-col-4"))
						]
						+ SResponsiveGridPanel::Slot(2)
						.ColumnSpan(SResponsiveGridSize::Mobile, 6)
						[
							ConstructBox(TEXT(".mobile-col-6"))
						]
						+ SResponsiveGridPanel::Slot(2)
						.ColumnSpan(SResponsiveGridSize::Mobile, 6)
						[
							ConstructBox(TEXT(".mobile-col-6"))
						]
					]
				]
			]

			+ SScrollBox::Slot()
			.Padding(0, 20, 0, 0)
			[
				SNew(SBox)
				.Padding(FMargin(50, 10))
				[
					SNew(SBorder)
					.BorderImage(FTestStyle::Get().GetBrush("RichText.Tagline.DarkBackground"))
					.Padding(10.0f)
					[
						SNew(SResponsiveGridPanel, 12)
						.ColumnGutter(5)
						.RowGutter(5)
						+ SResponsiveGridPanel::Slot(0)
						.ColumnSpan(SResponsiveGridSize::MediumDevice, 8)
						.ColumnSpan(SResponsiveGridSize::Tablet, 6)
						.ColumnSpan(SResponsiveGridSize::Mobile, 12)
						[
							ConstructBox(TEXT(".mobile-col-12 .tablet-col-6 .desktop-col-8"))
						]
						+ SResponsiveGridPanel::Slot(0)
						.ColumnSpan(SResponsiveGridSize::MediumDevice, 4)
						.ColumnSpan(SResponsiveGridSize::Mobile, 6)
						[
							ConstructBox(TEXT(".mobile-col-6 .desktop-col-4"))
						]
						+ SResponsiveGridPanel::Slot(1)
						.ColumnSpan(SResponsiveGridSize::Tablet, 4)
						.ColumnSpan(SResponsiveGridSize::Mobile, 6)
						[
							ConstructBox(TEXT(".mobile-col-6 .tablet-col-4"))
						]
						+ SResponsiveGridPanel::Slot(1)
						.ColumnSpan(SResponsiveGridSize::Tablet, 4)
						.ColumnSpan(SResponsiveGridSize::Mobile, 6)
						[
							ConstructBox(TEXT(".mobile-col-6 .tablet-col-4"))
						]
						+ SResponsiveGridPanel::Slot(1)
						.ColumnSpan(SResponsiveGridSize::Tablet, 4)
						.ColumnSpan(SResponsiveGridSize::Mobile, 6)
						[
							ConstructBox(TEXT(".mobile-col-6 .tablet-col-4"))
						]
					]
				]
			]

			+ SScrollBox::Slot()
			.Padding(0, 20, 0, 0)
			[
				SNew(SBox)
				.Padding(FMargin(50, 10))
				[
					SNew(SBorder)
					.BorderImage(FTestStyle::Get().GetBrush("RichText.Tagline.DarkBackground"))
					.Padding(10.0f)
					[
						SNew(SResponsiveGridPanel, 12)
						.ColumnGutter(5)
						.RowGutter(5)
						+ SResponsiveGridPanel::Slot(0)
						.ColumnSpan(SResponsiveGridSize::Mobile, 9)
						[
							ConstructBox(TEXT(".mobile-col-9"))
						]
						+ SResponsiveGridPanel::Slot(0)
						.ColumnSpan(SResponsiveGridSize::Mobile, 4)
						[
							ConstructBox(TEXT(".mobile-col-4 \nSince 9 + 4 = 13 > 12, this 4 column wide slot\ngets wrapped onto a new line as one contiguous unit."))
						]
						+ SResponsiveGridPanel::Slot(0)
						.ColumnSpan(SResponsiveGridSize::Mobile, 6)
						[
							ConstructBox(TEXT(".mobile-col-6 \nSubsequent columns continue along the new line."))
						]
					]
				]
			]

			+ SScrollBox::Slot()
			.Padding(0, 20, 0, 0)
			[
				SNew(SBox)
				.Padding(FMargin(50, 10))
				[
					SNew(SBorder)
					.BorderImage(FTestStyle::Get().GetBrush("RichText.Tagline.DarkBackground"))
					.Padding(10.0f)
					[
						SNew(SResponsiveGridPanel, 12)
						.ColumnGutter(5)
						.RowGutter(5)
						+ SResponsiveGridPanel::Slot(0)
						.ColumnSpan(SResponsiveGridSize::MediumDevice, 4)
						[
							ConstructBox(TEXT(".desktop-col-4"))
						]
						+ SResponsiveGridPanel::Slot(0)
						.ColumnSpan(SResponsiveGridSize::MediumDevice, 4, 4)
						[
							ConstructBox(TEXT(".desktop-col-4-offset-4"))
						]
						+ SResponsiveGridPanel::Slot(1)
						.ColumnSpan(SResponsiveGridSize::MediumDevice, 3, 3)
						[
							ConstructBox(TEXT(".desktop-col-3-offset-3"))
						]
						+ SResponsiveGridPanel::Slot(1)
						.ColumnSpan(SResponsiveGridSize::MediumDevice, 3, 3)
						[
							ConstructBox(TEXT(".desktop-col-3-offset-3"))
						]
						+ SResponsiveGridPanel::Slot(2)
						.ColumnSpan(SResponsiveGridSize::MediumDevice, 6, 3)
						[
							ConstructBox(TEXT(".desktop-col-6-offset-3"))
						]
					]
				]
			]

			+ SScrollBox::Slot()
			.Padding(0, 20, 0, 0)
			[
				SNew(SBox)
				.Padding(FMargin(50, 10))
				.HeightOverride(800)
				[
					SNew(SBorder)
					.BorderImage(FTestStyle::Get().GetBrush("RichText.Tagline.DarkBackground"))
					.Padding(10.0f)
					[
						SNew(SResponsiveGridPanel, 12)
						.ColumnGutter(5)
						.RowGutter(5)
						.FillRow(1, 1.0f)
						+ SResponsiveGridPanel::Slot(0)
						.ColumnSpan(SResponsiveGridSize::MediumDevice, 4)
						[
							ConstructBox(TEXT(".desktop-col-4"))
						]
						+ SResponsiveGridPanel::Slot(0)
						.ColumnSpan(SResponsiveGridSize::MediumDevice, 4)
						[
							ConstructBox(TEXT(".desktop-col-4"))
						]
						+ SResponsiveGridPanel::Slot(0)
						.ColumnSpan(SResponsiveGridSize::MediumDevice, 4)
						[
							ConstructBox(TEXT(".desktop-col-4"))
						]
						+ SResponsiveGridPanel::Slot(1)
						.ColumnSpan(SResponsiveGridSize::MediumDevice, 3)
						.ColumnSpan(SResponsiveGridSize::Mobile, 0)
						[
							ConstructBox(TEXT(".desktop-col-3"))
						]
						+ SResponsiveGridPanel::Slot(1)
						.ColumnSpan(SResponsiveGridSize::MediumDevice, 9)
						[
							ConstructBox(TEXT(".desktop-col-9"))
						]
						+ SResponsiveGridPanel::Slot(2)
						.ColumnSpan(SResponsiveGridSize::MediumDevice, 6, 3)
						[
							ConstructBox(TEXT(".desktop-col-6-offset-3"))
						]
					]
				]
			]
		]
	]);

	for (int32 Index = 0; Index < 12; ++Index)
	{
		GridPanel->AddSlot(0)
		.ColumnSpan(SResponsiveGridSize::MediumDevice, 1)
		[
			ConstructBox(TEXT(".desktop-col-1"))
		];
	}

	GridPanel->AddSlot(1)
	.ColumnSpan(SResponsiveGridSize::MediumDevice, 8)
	[
		ConstructBox(TEXT(".desktop-col-8"))
	];

	GridPanel->AddSlot(1)
	.ColumnSpan(SResponsiveGridSize::MediumDevice, 4)
	[
		ConstructBox(TEXT(".desktop-col-4"))
	];

	for (int32 Index = 0; Index < 3; ++Index)
	{
		GridPanel->AddSlot(2)
		.ColumnSpan(SResponsiveGridSize::MediumDevice, 4)
		[
			ConstructBox(TEXT(".desktop-col-4"))
		];
	}
	
	for (int32 Index = 0; Index < 2; ++Index)
	{
		GridPanel->AddSlot(3)
		.ColumnSpan(SResponsiveGridSize::MediumDevice, 6)
		[
			ConstructBox(TEXT(".desktop-col-6"))
		];
	}
}

TSharedRef<SResponsiveGridPanelTestWidget> SResponsiveGridPanelTestWidget::New()
{
	return MakeShareable(new SResponsiveGridPanelTestWidgetImpl());
}

/**
 * User Widget wrapper for a this widget. Legacy as this widget used to be in a separate header.
 */
class SRenderTransformManipulatorWidget : public SUserWidget
{

public:

	SLATE_USER_ARGS(SRenderTransformManipulatorWidget)
	{}

	SLATE_END_ARGS()

	virtual void Construct(const FArguments& InArgs) = 0;
};

namespace TestSuiteNS
{
	float RotDeg = 0.0f;
	FScale2D Scale;
	FShear2D Shear;
	FQuat2D Rot;
	FVector2f Offset(0,0);
}

class SRenderTransformManipulatorWidgetImpl : public SRenderTransformManipulatorWidget
{
public:
	virtual void Construct(const FArguments& InArgs) override;

	TSharedPtr<SImage> ImageWidget;
private:
	static const ISlateStyle& GetStyle()
	{
		static FSlateStyleSet Style("RenderTransformManipulatorStyle");
		static bool IsInit = false;
		if (!IsInit)
		{
			check(IsInGameThread());
			Style.SetContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
			Style.Set("UEIcon", new FSlateVectorImageBrush(Style.RootToContentDir(TEXT("Starship/Common/unreal.svg")), FVector2D(50, 50)));
			IsInit = true;
		}
		return Style;
	}
};

/**
 * Global access here because we need other translation units to access this function.
 */
TOptional<FSlateRenderTransform> GetTestRenderTransform()
{
	return TransformCast<FSlateRenderTransform>(Concatenate(TestSuiteNS::Shear, TestSuiteNS::Scale, TestSuiteNS::Rot));
}

FVector2D GetTestRenderTransformPivot()
{
	return FVector2D(TestSuiteNS::Offset);
}

void SRenderTransformManipulatorWidgetImpl::Construct(const FArguments& InArgs)
{
	SUserWidget::Construct(SUserWidget::FArguments()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot().Padding(4)
				[
					SAssignNew(ImageWidget, SImage)
					.RenderTransform_Static(&::GetTestRenderTransform)
					.RenderTransformPivot_Static(&::GetTestRenderTransformPivot)
					.Image(GetStyle().GetBrush("UEIcon"))
				]
			+ SVerticalBox::Slot().AutoHeight().Padding(4)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(STextBlock).Text(LOCTEXT("RD", "Rotation degrees"))
				]
				+ SHorizontalBox::Slot().HAlign(HAlign_Fill)
					[
						SNew(SSpinBox<float>)
						.MinValue(0.0f)
						.MaxValue(360.0f)
						.OnValueChanged_Static([](float val) { TestSuiteNS::RotDeg = val; TestSuiteNS::Rot = FQuat2D(FMath::DegreesToRadians(val)); })
						.Value_Static([] { return TestSuiteNS::RotDeg; })
					]
			]
			+ SVerticalBox::Slot().AutoHeight().Padding(4)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						SNew(STextBlock).Text(LOCTEXT("XO", "X Offset"))
					]
					+ SHorizontalBox::Slot().HAlign(HAlign_Fill)
						[
							SNew(SSpinBox<float>)
							.MinValue(-2.0f)
							.MaxValue(2.0f)
							.OnValueChanged_Static([](float val) { TestSuiteNS::Offset.X = val; })
							.Value_Static([] { return TestSuiteNS::Offset.X; })
						]
				]
			+ SVerticalBox::Slot().AutoHeight().Padding(4)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						SNew(STextBlock).Text(LOCTEXT("YO", "Y Offset"))
					]
					+ SHorizontalBox::Slot().HAlign(HAlign_Fill)
						[
							SNew(SSpinBox<float>)
							.MinValue(-2.0f)
							.MaxValue(2.0f)
							.OnValueChanged_Static([](float val) { TestSuiteNS::Offset.Y = val; })
							.Value_Static([] { return TestSuiteNS:: Offset.Y; })
						]
				]
			+ SVerticalBox::Slot().AutoHeight().Padding(4)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						SNew(STextBlock).Text(LOCTEXT("XS", "X Scale"))
					]
					+ SHorizontalBox::Slot().HAlign(HAlign_Fill)
						[
							SNew(SSpinBox<float>)
							.MinValue(-10.0f)
							.MaxValue(10.0f)
							.OnValueChanged_Static([](float val) { TestSuiteNS::Scale = FScale2D(val, TestSuiteNS::Scale.GetVector().Y); })
							.Value_Static([] { return TestSuiteNS::Scale.GetVector().X; })
						]
				]
			+ SVerticalBox::Slot().AutoHeight().Padding(4)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						SNew(STextBlock).Text(LOCTEXT("YS", "Y Scale"))
					]
					+ SHorizontalBox::Slot().HAlign(HAlign_Fill)
						[
							SNew(SSpinBox<float>)
							.MinValue(-10.0f)
							.MaxValue(10.0f)
							.OnValueChanged_Static([](float val) { TestSuiteNS::Scale = FScale2D(TestSuiteNS::Scale.GetVector().X, val); })
							.Value_Static([] { return TestSuiteNS::Scale.GetVector().Y; })
						]
				]
			+ SVerticalBox::Slot().AutoHeight().Padding(4)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						SNew(STextBlock).Text(LOCTEXT("XSH", "X Shear"))
					]
					+ SHorizontalBox::Slot().HAlign(HAlign_Fill)
						[
							SNew(SSpinBox<float>)
							.MinValue(-4.f)
							.MaxValue(4.f)
							.OnValueChanged_Static([](float val) { TestSuiteNS::Shear = FShear2D(val, TestSuiteNS::Shear.GetVector().Y); })
							.Value_Static([] { return TestSuiteNS::Shear.GetVector().X; })
						]
				]
			+ SVerticalBox::Slot().AutoHeight().Padding(4)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						SNew(STextBlock).Text(LOCTEXT("YSH", "Y Shear"))
					]
					+ SHorizontalBox::Slot().HAlign(HAlign_Fill)
						[
							SNew(SSpinBox<float>)
							.MinValue(-4.f)
							.MaxValue(4.f)
							.OnValueChanged_Static([](float val) { TestSuiteNS::Shear = FShear2D(TestSuiteNS::Shear.GetVector().X, val); })
							.Value_Static([] { return TestSuiteNS::Shear.GetVector().Y; })
						]
				]
		]
	);
}

/**
 * New implementation for the User Widget idiom.
 */
TSharedRef<SRenderTransformManipulatorWidget> SRenderTransformManipulatorWidget::New()
{
	return MakeShareable(new SRenderTransformManipulatorWidgetImpl());
}

static TSharedPtr<FTabManager> TestSuite1TabManager;

TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args, FName TabIdentifier)
{
	if (TabIdentifier == FName("AnimationTestTab"))
	{
		return SNew(SDockTab)
			.Label( LOCTEXT("AnimationTestTabLabel", "Animation Test") )
			.Clipping(EWidgetClipping::ClipToBounds)
			. ToolTip
			(
				SNew(SToolTip)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding( 2.0f )
					[
						SNew(SImage)
						.Image( FCoreStyle::Get().GetBrush( TEXT("AppIcon") ) )
					]
					+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign( VAlign_Center )
					[
						SNew(STextBlock)
							.Text( ( LOCTEXT("AnimationTestLabel", "Bring up some test for animation.") ) )
					]
				]
			)
		[
			SNew(SVerticalBox)
			.RenderTransform_Static(&::GetTestRenderTransform)
			.RenderTransformPivot_Static(&::GetTestRenderTransformPivot)

			+SVerticalBox::Slot()
			[
				SNew(SAnimTest)
			]

			+SVerticalBox::Slot()
			[
				SNew(SFxTest)
			]
		];
	}
	else if ( TabIdentifier == FName("DocumentsTestTab") )
	{
		FTabManager* TabManagerRef = &(TestSuite1TabManager.ToSharedRef().Get());

		return
			SNew(SDockTab)
			.Label(NSLOCTEXT("TestSuite1", "DocumentsTab", "Documents"))
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew(SDocumentsTest, TabManagerRef)
				.RenderTransform_Static(&::GetTestRenderTransform)
				.RenderTransformPivot_Static(&::GetTestRenderTransformPivot)
				.AddMetaData<FTagMetaData>(FTagMetaData("DocumentSpawner"))
			];
	}
	else if (TabIdentifier == FName("TableViewTestTab"))
	{
		return SNew(SDockTab)
			.Label( LOCTEXT("ListViewTestTab", "ListView Test") )
			.ToolTipText( LOCTEXT( "ListViewTestToolTip", "Switches to the List View test tab, which allows you to test list widgets in Slate." ) )
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				MakeTableViewTesting()
			];
	}
	else if (TabIdentifier == FName("LayoutExampleTab"))
	{
		return SNew(SDockTab)
			.Label( LOCTEXT("LayoutExampleTab", "Layout Example") )
			.ToolTipText( LOCTEXT( "LayoutExampleTabToolTip", "Switches to the Layout Example tab, which shows off examples of various Slate layout primitives." ) )
			.Clipping(EWidgetClipping::ClipToBounds)
		[
			MakeLayoutExample()
		];
	}
#if WITH_FANCY_TEXT
	else if (TabIdentifier == FName("RichTextTab"))
	{
		return SNew(SDockTab)
			.Label( LOCTEXT("RichTextTestTab", "Rich Text") )
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew( SRichTextTest )
				.RenderTransform_Static(&::GetTestRenderTransform)
				.RenderTransformPivot_Static(&::GetTestRenderTransformPivot)
			];
	}
	else if ( TabIdentifier == FName( "MultiLineEditTab" ) )
	{
		return SNew(SDockTab)
			.Label( LOCTEXT("MultiLineEditTab", "MultiLine Edit") )
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				#if WITH_FANCY_TEXT
				SNew( SMultiLineEditingTest )
				#else
				SNew( SSpacer )
				#endif //WITH_FANCY_TEXT
				.RenderTransform_Static(&::GetTestRenderTransform)
				.RenderTransformPivot_Static(&::GetTestRenderTransformPivot)
			];
	}
#endif //WITH_FANCY_TEXT
	else if (TabIdentifier == FName("EditableTextTab"))
	{
		return SNew(SDockTab)
			.Label( LOCTEXT("EditableTextTestTab", "Editable Text") )
			.ToolTipText( LOCTEXT( "EditableTextTestTabToolTip", "Switches to the Editable Text tab, where you can test the various inline text editing controls." ) )
			.Clipping(EWidgetClipping::ClipToBounds)
		[
			SNew( STextEditTest )
			.RenderTransform_Static(&::GetTestRenderTransform)
			.RenderTransformPivot_Static(&::GetTestRenderTransformPivot)
		];
	}
#if WITH_FANCY_TEXT
	else if (TabIdentifier == FName("RichEditableTextTab"))
	{
		return SNew(SDockTab)
			.Label( LOCTEXT("RichEditableTextTestTab", "Rich Editable Text") )
			.ToolTipText( LOCTEXT( "RichEditableTextTestTabToolTip", "Switches to the Rich Editable Text tab, where you can test the various rich editable text features." ) )
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew( SRichTextEditTest )
				.RenderTransform_Static(&::GetTestRenderTransform)
				.RenderTransformPivot_Static(&::GetTestRenderTransformPivot)
			];
	}
#endif //WITH_FANCY_TEXT
	else if (TabIdentifier == FName("LayoutRoundingTab"))
	{
		return SNew(SDockTab)
			. Label( LOCTEXT("LayoutRoundingTab", "Layout Rounding") )
			.Clipping(EWidgetClipping::ClipToBounds)
		[
			SNew( SLayoutRoundingTest )
			.RenderTransform_Static(&::GetTestRenderTransform)
			.RenderTransformPivot_Static(&::GetTestRenderTransformPivot)
		];
	}
	else if (TabIdentifier == FName("SplineTestTab"))
	{
		return SNew(SDockTab)
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew(SUniformGridPanel)
				.RenderTransform_Static(&::GetTestRenderTransform)
				.RenderTransformPivot_Static(&::GetTestRenderTransformPivot)
				.SlotPadding(FMargin(10.0f))
				+SUniformGridPanel::Slot(0,0)
				[
					SNew(SBorder)
					.Clipping(EWidgetClipping::ClipToBounds)
					[
						SNew(SSplineTest)
						.P0(FVector2f(64, 64))
						.P1(FVector2f(256, 64))
						.P2(FVector2f(64, 256))
						.P3(FVector2f(256, 256))
					]
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SBorder)
					.Clipping(EWidgetClipping::ClipToBounds)
					[
						SNew(SSplineTest)
						.P0(FVector2f(128 - 32, 32))
						.P1(FVector2f(128 + 32, 128))
						.P2(FVector2f(128 - 32, 128))
						.P3(FVector2f(128 + 32, 32))
					]
				]
				+ SUniformGridPanel::Slot(0, 1)
				[
					SNew(SBorder)
					.Clipping(EWidgetClipping::ClipToBounds)
					[
						SNew(SSplineTest)
						.P0(FVector2f(64, 64))
						.P1(FVector2f(256, 64))
						.P2(FVector2f(256, 256))
						.P3(FVector2f(64, 256))
					]
				]
				+ SUniformGridPanel::Slot(1, 1)
				[
					SNew(SBorder)
					.Clipping(EWidgetClipping::ClipToBounds)
					[
						SNew(SSplineTest)
						.P0(FVector2f(64, 64))
						.P1(FVector2f(128, 128))
						.P2(FVector2f(196, 196))
						.P3(FVector2f(256, 256))
					]
				]
				
			];
	}
	else if (TabIdentifier == FName("ElementTestsTab"))
	{
		return SNew(SDockTab)
			. Label( LOCTEXT("ElementTestsTab", "Element Tests") )
			. ToolTipText( LOCTEXT( "ElementTestsTabToolTip", "Switches to the Element Tests tab, which allows you to view various rendering-related features of Slate." ) )
			.Clipping(EWidgetClipping::ClipToBounds)
		[
			SNew( SElementTesting )
			.RenderTransform_Static(&::GetTestRenderTransform)
			.RenderTransformPivot_Static(&::GetTestRenderTransformPivot)
		];
	}
	else if (TabIdentifier == FName("SplitterTestTab"))
	{
		return SNew(SDockTab)
			.Label( LOCTEXT("SplitterTestTab", "Splitter Test") )
			.ToolTipText( LOCTEXT( "SplitterTestTabToolTip", "Switches to the Splitter Test tab, which you can use to test splitters." ) )
			.Clipping(EWidgetClipping::ClipToBounds)
		[
			SNew( SSplitterTest )
			.RenderTransform_Static(&::GetTestRenderTransform)
			.RenderTransformPivot_Static(&::GetTestRenderTransformPivot)
		];
	}
	else if (TabIdentifier == FName("MultiBoxTestTab"))
	{
		return SNew(SDockTab)
			.Label( LOCTEXT("MultiBoxTextTab", "MultiBox Text") )
			.ToolTipText( LOCTEXT( "MultiBoxTextTabToolTip", "Switches to the MultiBox tab, where you can test out MultiBoxes and MultiBlocks." ) )
			.Clipping(EWidgetClipping::ClipToBounds)
		[
			SNew( SMultiBoxTest )
			.RenderTransform_Static(&::GetTestRenderTransform)
			.RenderTransformPivot_Static(&::GetTestRenderTransformPivot)
		];
	}
	else if (TabIdentifier == FName("ColorPickerTestTab"))
	{
		return SNew(SDockTab)
			.Label( LOCTEXT("ColorPickerTestTab", "Color Picker Test") )
			.ToolTipText( LOCTEXT( "ColorPickerTestTabToolTip", "Switches to the Color Picker tab, where you can test out the color picker." ) )
			.Clipping(EWidgetClipping::ClipToBounds)
		[
			SNew(SColorPickerTest)
			.RenderTransform_Static(&::GetTestRenderTransform)
			.RenderTransformPivot_Static(&::GetTestRenderTransformPivot)
		];
	}
	else if (TabIdentifier == "DPIScalingTest")
	{
		return SNew(SDockTab)
			.Clipping(EWidgetClipping::ClipToBounds)
		[
			SNew(SDPIScalingTest)
			.RenderTransform_Static(&::GetTestRenderTransform)
			.RenderTransformPivot_Static(&::GetTestRenderTransformPivot)
		];
	}
	else if ( TabIdentifier == FName("InvalidationTest") )
	{
		return SNew(SDockTab)
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew(SInvalidationTest)
				.RenderTransform_Static(&::GetTestRenderTransform)
				.RenderTransformPivot_Static(&::GetTestRenderTransformPivot)
			];
	}
	else if ( TabIdentifier == FName("GammaTest") )
	{
		return SNew(SDockTab)
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew(SGammaTest)
				.RenderTransform_Static(&::GetTestRenderTransform)
				.RenderTransformPivot_Static(&::GetTestRenderTransformPivot)
			];
	}
	else if (TabIdentifier == FName("NotificationListTestTab"))
	{
		return SNew(SDockTab)
			.Label( LOCTEXT("NotificationListTestTab", "Notification List Test") )
			.ToolTipText( LOCTEXT( "NotificationListTestTabToolTip", "Switches to the Notification List tab, where you can test out the notification list." ) )
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew(SNotificationListTest)
				.RenderTransform_Static(&::GetTestRenderTransform)
				.RenderTransformPivot_Static(&::GetTestRenderTransformPivot)
			];
	}
	else if (TabIdentifier == FName("GridPanelTest"))
	{
		return SNew(SDockTab)
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				SNew(SGridPanelTest)
				.RenderTransform_Static(&::GetTestRenderTransform)
				.RenderTransformPivot_Static(&::GetTestRenderTransformPivot)
			];
	}
	else
	{
		ensure(false);
		return SNew(SDockTab);
	}
}

TSharedRef<SDockTab> SpawnResponsiveGrid(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> ResponsiveGridTab =
		SNew(SDockTab)
		.Label(LOCTEXT("ResponsiveGridTabLabel", "Responsive Grid"));

	ResponsiveGridTab->SetContent
		(
			SNew(SResponsiveGridPanelTestWidget)
		);

	return ResponsiveGridTab;
}



namespace TestSuiteMenu
{
	TSharedRef<FWorkspaceItem> MenuRoot = FWorkspaceItem::NewGroup(NSLOCTEXT("TestSuite", "MenuRoot", "MenuRoot") );
	TSharedRef<FWorkspaceItem> SuiteTabs = MenuRoot->AddGroup( NSLOCTEXT("TestSuite", "SuiteTabs", "Test Suite Tabs") );
	TSharedRef<FWorkspaceItem> NestedCategory = SuiteTabs->AddGroup( NSLOCTEXT("TestSuite", "NestedCategory", "Nested") );
	TSharedRef<FWorkspaceItem> DeveloperCategory = MenuRoot->AddGroup( NSLOCTEXT("TestSuite", "DeveloperCategory", "Developer") );
}


TSharedRef<SDockTab> SpawnTestSuite1( const FSpawnTabArgs& Args )
{
	TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout( "TestSuite1_Layout" )
	->AddArea
	(
		// The primary area will be restored and returned as a widget.
		// Unlike other areas it will not get its own window.
		// This allows the caller to explicitly place the primary area somewhere in the widget hierarchy.
		FTabManager::NewPrimaryArea()
		->Split
		(
			//The first cell in the primary area will be occupied by a stack of tabs.
			// They are all opened.
			FTabManager::NewStack()
			->SetSizeCoefficient(0.2f)
			->AddTab("LayoutExampleTab", ETabState::OpenedTab)
			->AddTab("DocumentsTestTab", ETabState::OpenedTab)
		)
		->Split
		(
			// We can subdivide a cell further by using an additional splitter
			FTabManager::NewSplitter()
			->SetOrientation( Orient_Vertical )
			->SetSizeCoefficient(0.8f)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.25f)
				->AddTab("MultiBoxTestTab", ETabState::OpenedTab)
				->AddTab("ResponsiveGrid", ETabState::OpenedTab)
				// The DocTest tab will be closed by default.
				->AddTab("DocTest", ETabState::ClosedTab)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.75f)
				#if PLATFORM_SUPPORTS_MULTIPLE_NATIVE_WINDOWS
				->AddTab("TableViewTestTab", ETabState::OpenedTab)
				->SetForegroundTab(FName("TableViewTestTab"))
				#endif
				->AddTab("LayoutRoundingTab", ETabState::OpenedTab)
				->AddTab("EditableTextTab", ETabState::OpenedTab)
#if WITH_FANCY_TEXT
				->AddTab("RichTextTab", ETabState::OpenedTab)
				->AddTab("MultiLineEditTab", ETabState::OpenedTab)
				->AddTab("RichEditableTextTab", ETabState::OpenedTab)
				->AddTab("WebBrowserTab", ETabState::OpenedTab)
#endif //WITH_FANCY_TEXT
				)
		)
	)
#if PLATFORM_SUPPORTS_MULTIPLE_NATIVE_WINDOWS
	->AddArea
	(
		FTabManager::NewArea(320,240)
		->SetWindow( FVector2D(600, 50), false )
		->Split
		(
			FTabManager::NewStack()
			->AddTab("SplitterTestTab", ETabState::ClosedTab)
		)
	)
#endif
	;



	TSharedRef<SDockTab> TestSuite1Tab =
		SNew(SDockTab)
		. TabRole( ETabRole::MajorTab )
		. Label( LOCTEXT("TestSuite1TabLabel", "Test Suite 1") )
		. ToolTipText( LOCTEXT( "TestSuite1TabToolTip", "The App for the first Test Suite." ) );

	
	TestSuite1TabManager = FGlobalTabmanager::Get()->NewTabManager(TestSuite1Tab);
	TestSuite1TabManager->RegisterTabSpawner( "LayoutExampleTab", FOnSpawnTab::CreateStatic( &SpawnTab, FName("LayoutExampleTab") ) )
		.SetDisplayName( NSLOCTEXT("TestSuite1", "LayoutExampleTab", "Layout Example") )
		.SetGroup(TestSuiteMenu::SuiteTabs);

	TestSuite1TabManager->RegisterTabSpawner( "SplitterTestTab", FOnSpawnTab::CreateStatic( &SpawnTab, FName("SplitterTestTab") ) )
		.SetDisplayName( NSLOCTEXT("TestSuite1", "SplitterTestTab", "Splitter Test") )
		.SetGroup(TestSuiteMenu::SuiteTabs);

	TestSuite1TabManager->RegisterTabSpawner( "EditableTextTab", FOnSpawnTab::CreateStatic( &SpawnTab, FName("EditableTextTab") ) )
		.SetDisplayName( NSLOCTEXT("TestSuite1", "EditableTextTab", "Editable Text Test") )
		.SetGroup(TestSuiteMenu::SuiteTabs);

	TestSuite1TabManager->RegisterTabSpawner( "RichTextTab", FOnSpawnTab::CreateStatic( &SpawnTab, FName("RichTextTab") ) )
		.SetDisplayName( NSLOCTEXT("TestSuite1", "RichTextTab", "Rich Text Test") )
		.SetGroup(TestSuiteMenu::SuiteTabs);

	TestSuite1TabManager->RegisterTabSpawner( "MultiLineEditTab", FOnSpawnTab::CreateStatic( &SpawnTab, FName( "MultiLineEditTab" ) ) )
		.SetDisplayName( NSLOCTEXT( "TestSuite1", "MultiLineEditTab", "Multiline Edit" ) )
		.SetGroup( TestSuiteMenu::SuiteTabs );

	TestSuite1TabManager->RegisterTabSpawner( "RichEditableTextTab", FOnSpawnTab::CreateStatic( &SpawnTab, FName("RichEditableTextTab") ) )
		.SetDisplayName( NSLOCTEXT("TestSuite1", "RichEditableTextTab", "Rich Editable Text Test") )
		.SetGroup(TestSuiteMenu::SuiteTabs);

	TestSuite1TabManager->RegisterTabSpawner( "LayoutRoundingTab", FOnSpawnTab::CreateStatic( &SpawnTab, FName("LayoutRoundingTab") ) )
		.SetDisplayName( NSLOCTEXT("TestSuite1", "LayoutRoundingTab", "Layout Rounding") )
		.SetGroup(TestSuiteMenu::SuiteTabs);

	TestSuite1TabManager->RegisterTabSpawner("ResponsiveGrid", FOnSpawnTab::CreateStatic(&SpawnResponsiveGrid))
		.SetGroup(TestSuiteMenu::SuiteTabs);

	TestSuite1TabManager->RegisterTabSpawner( "MultiBoxTestTab", FOnSpawnTab::CreateStatic( &SpawnTab, FName("MultiBoxTestTab") ) )
		.SetDisplayName( NSLOCTEXT("TestSuite1", "MultiBoxTestTab", "MultiBox Test") )
		.SetGroup(TestSuiteMenu::NestedCategory);

	TestSuite1TabManager->RegisterTabSpawner( "TableViewTestTab", FOnSpawnTab::CreateStatic( &SpawnTab, FName("TableViewTestTab") ) )
		.SetDisplayName( NSLOCTEXT("TestSuite1", "TableViewTestTab", "TableView Test") )
		.SetGroup(TestSuiteMenu::NestedCategory);

	TestSuite1TabManager->RegisterTabSpawner( "DocumentsTestTab", FOnSpawnTab::CreateStatic( &SpawnTab, FName("DocumentsTestTab") ) )
		.SetDisplayName( NSLOCTEXT("TestSuite1", "DocumentsTestTab", "Document Spawner") );


	FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder( TSharedPtr<FUICommandList>() );
	MenuBarBuilder.AddPullDownMenu(
		NSLOCTEXT("TestSuite", "WindowMenuLabel", "Window"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateSP(TestSuite1TabManager.ToSharedRef(), &FTabManager::PopulateTabSpawnerMenu, TestSuiteMenu::MenuRoot));

	TestSuite1Tab->SetContent
	(
		SNew( SVerticalBox )
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			MenuBarBuilder.MakeWidget()
		]
		+SVerticalBox::Slot()
		.FillHeight( 1.0f )
		[
			TestSuite1TabManager->RestoreFrom(Layout, Args.GetOwnerWindow() ).ToSharedRef()
		]
	);

	return TestSuite1Tab;
}

static TSharedPtr<FTabManager> TestSuite2TabManager;


TSharedRef<SDockTab> SpawnTestSuite2( const FSpawnTabArgs& Args )
{	
	TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout( "TestSuite2_Layout" )
	->AddArea
	(
		FTabManager::NewPrimaryArea()
		->Split
		(
			FTabManager::NewStack()
			->AddTab("AnimationTestTab", ETabState::OpenedTab)
			->AddTab("ElementTestsTab", ETabState::OpenedTab)
			->AddTab("ColorPickerTestTab", ETabState::OpenedTab)
			->AddTab("NotificationListTestTab", ETabState::OpenedTab)
			->AddTab("GridPanelTest", ETabState::OpenedTab)
			->AddTab("DPIScalingTest", ETabState::OpenedTab)
			->AddTab("InvalidationTest", ETabState::OpenedTab)
			->AddTab("GammaTest", ETabState::OpenedTab)
			->AddTab("SplineTestTab", ETabState::OpenedTab)
		)
	);

	TSharedRef<SDockTab> TestSuite2Tab = 
		SNew(SDockTab)
		. TabRole( ETabRole::MajorTab )
		. Label( LOCTEXT("TestSuite2TabLabel", "Test Suite 2") )
		. ToolTipText( LOCTEXT("TestSuite2TabToolTip", "The App for the first Test Suite." ) );

	TestSuite2TabManager = FGlobalTabmanager::Get()->NewTabManager( TestSuite2Tab );
	{
		TestSuite2TabManager->RegisterTabSpawner( "AnimationTestTab", FOnSpawnTab::CreateStatic( &SpawnTab, FName("AnimationTestTab") )  )
			.SetDisplayName( NSLOCTEXT("TestSuite2", "AnimationTestTab", "Animation Test") )
			.SetGroup(TestSuiteMenu::SuiteTabs);

		TestSuite2TabManager->RegisterTabSpawner("SplineTestTab", FOnSpawnTab::CreateStatic(&SpawnTab, FName("SplineTestTab")))
			.SetDisplayName(NSLOCTEXT("TestSuite2", "SplineTestTab", "Spline Test"))
			.SetGroup(TestSuiteMenu::SuiteTabs);

		TestSuite2TabManager->RegisterTabSpawner( "ElementTestsTab", FOnSpawnTab::CreateStatic( &SpawnTab, FName("ElementTestsTab") )  )
			.SetDisplayName( NSLOCTEXT("TestSuite2", "ElementTestsTab", "Elements Test") )
			.SetGroup(TestSuiteMenu::SuiteTabs);

		TestSuite2TabManager->RegisterTabSpawner( "ColorPickerTestTab", FOnSpawnTab::CreateStatic( &SpawnTab, FName("ColorPickerTestTab") )  )
			.SetDisplayName( NSLOCTEXT("TestSuite2", "ColorPickerTestTab", "Color Picker Test") )
			.SetGroup(TestSuiteMenu::SuiteTabs);

		TestSuite2TabManager->RegisterTabSpawner( "NotificationListTestTab", FOnSpawnTab::CreateStatic( &SpawnTab, FName("NotificationListTestTab") )  )
			.SetDisplayName( NSLOCTEXT("TestSuite2", "NotificationListTestTab", "Notifications Test") )
			.SetGroup(TestSuiteMenu::SuiteTabs);

		TestSuite2TabManager->RegisterTabSpawner( "GridPanelTest", FOnSpawnTab::CreateStatic( &SpawnTab, FName("GridPanelTest") )  )
			.SetDisplayName( NSLOCTEXT("TestSuite2", "GridPanelTest", "Grid Panel") )
			.SetGroup(TestSuiteMenu::SuiteTabs);

		TestSuite2TabManager->RegisterTabSpawner( "DPIScalingTest", FOnSpawnTab::CreateStatic( &SpawnTab, FName("DPIScalingTest") )  )
			.SetDisplayName( NSLOCTEXT("TestSuite2", "DPIScalingTest", "DPI Scaling") )
			.SetGroup(TestSuiteMenu::SuiteTabs);

		TestSuite2TabManager->RegisterTabSpawner("InvalidationTest", FOnSpawnTab::CreateStatic(&SpawnTab, FName("InvalidationTest")))
			.SetDisplayName(NSLOCTEXT("TestSuite2", "InvalidationTest", "Invalidation"))
			.SetGroup(TestSuiteMenu::SuiteTabs);
		
		TestSuite2TabManager->RegisterTabSpawner("GammaTest", FOnSpawnTab::CreateStatic(&SpawnTab, FName("GammaTest")))
			.SetDisplayName(NSLOCTEXT("TestSuite2", "GammaTest", "Gamma"))
			.SetGroup(TestSuiteMenu::SuiteTabs);
	}

	FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder( TSharedPtr<FUICommandList>() );
	MenuBarBuilder.AddPullDownMenu(
		NSLOCTEXT("TestSuite", "WindowMenuLabel", "Window"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateSP(TestSuite2TabManager.ToSharedRef(), &FTabManager::PopulateTabSpawnerMenu, TestSuiteMenu::MenuRoot));

	TestSuite2Tab->SetContent
	(
		SNew( SVerticalBox )
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			MenuBarBuilder.MakeWidget()
		]
		+SVerticalBox::Slot()
		.FillHeight( 1.0f )
		[
			TestSuite2TabManager->RestoreFrom( Layout, Args.GetOwnerWindow() ).ToSharedRef()
		]
	);

	return TestSuite2Tab;
}

TSharedRef<SDockTab> SpawnRenderTransformManipulator( const FSpawnTabArgs& Args )
{	
	TSharedRef<SDockTab> RenderTransformManipulatorTab =
		SNew(SDockTab)
		.TabRole(ETabRole::MajorTab)
		.Label(LOCTEXT("RenderTransformTabLabel", "Render Transform"))
		.ToolTipText(LOCTEXT("RenderTransformTabToolTip", "Allows manipulating the render transform of all test tabs at a global level."));

	RenderTransformManipulatorTab->SetContent
	(
		SNew( SRenderTransformManipulatorWidget )
	);

	return RenderTransformManipulatorTab;
}

TSharedRef<SDockTab> SpawnWidgetGallery(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.Label(LOCTEXT("WidgetGalleryTab", "Widget Gallery"))
		.ToolTipText(LOCTEXT("WidgetGalleryTabTextToolTip", "Switch to the widget gallery."))
		.Clipping(EWidgetClipping::ClipToBounds)
		[
			MakeWidgetGallery()
		];
}

void RestoreSlateTestSuite()
{
	// Need to load this module so we have the widget reflector tab available
	FModuleManager::LoadModuleChecked<ISlateReflectorModule>("SlateReflector");

	FTestStyle::ResetToDefault();

	FGlobalTabmanager::Get()->RegisterTabSpawner("TestSuite1", FOnSpawnTab::CreateStatic( &SpawnTestSuite1 ) );
	FGlobalTabmanager::Get()->RegisterTabSpawner("TestSuite2", FOnSpawnTab::CreateStatic( &SpawnTestSuite2 ) );
	FGlobalTabmanager::Get()->RegisterTabSpawner("RenderTransformManipulator", FOnSpawnTab::CreateStatic(&SpawnRenderTransformManipulator));
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner("WidgetGalleryTab", FOnSpawnTab::CreateStatic(&SpawnWidgetGallery))
		.SetDisplayName(LOCTEXT("WidgetGalleryTab", "Widget Gallery"))
		.SetGroup(TestSuiteMenu::MenuRoot);

	TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout( "SlateTestSuite_Layout" )
	->AddArea
	(
		FTabManager::NewArea(1600, 1200)
#if PLATFORM_MAC
		->SetWindow( FVector2D(420, 32), false )
#else
		->SetWindow( FVector2D(420, 10), false )
#endif
		->Split
		(
			FTabManager::NewStack()
			->AddTab( "TestSuite2", ETabState::OpenedTab )
			->AddTab( "TestSuite1", ETabState::OpenedTab )
			->AddTab("RenderTransformManipulator", ETabState::OpenedTab)
			->AddTab("WidgetGalleryTab", ETabState::OpenedTab)
			->SetForegroundTab(FName("TestSuite1"))
		)		
	)
	#if PLATFORM_SUPPORTS_MULTIPLE_NATIVE_WINDOWS
	->AddArea
	(
		// This area will get a 400x600 window at 10,10
		FTabManager::NewArea(400, 600)
#if PLATFORM_MAC
		->SetWindow( FVector2D(10, 32), false )
#else
		->SetWindow( FVector2D(10, 10), false )
#endif
		->Split
		(
			// The area contains a single tab with the widget reflector.
			FTabManager::NewStack()->AddTab( "WidgetReflector", ETabState::OpenedTab )
		)
	)
	#endif
	;


	FGlobalTabmanager::Get()->RestoreFrom( Layout, TSharedPtr<SWindow>() );
}

void MakeSplitterTest()
{
	TSharedRef<SWindow> TestWindow = SNew(SWindow)
	.ClientSize(FVector2D(640,480))
	.AutoCenter(EAutoCenter::PrimaryWorkArea)
	[
		SNew(SSplitterTest)
	];

	FSlateApplication::Get().AddWindow( TestWindow );
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef LOCTEXT_NAMESPACE

#endif // #if !UE_BUILD_SHIPPING
