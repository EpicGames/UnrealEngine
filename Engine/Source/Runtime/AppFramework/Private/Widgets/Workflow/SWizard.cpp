// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Workflow/SWizard.h"
#include "Widgets/SBoxPanel.h"
#include "Application/SlateWindowHelper.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#define LOCTEXT_NAMESPACE "SWizard"


/* SWizard interface
 *****************************************************************************/

bool SWizard::CanShowPage( int32 PageIndex ) const
{
	if (Pages.IsValidIndex(PageIndex))
	{
		return Pages[PageIndex].CanShow();
	}

	return false;
}

void SWizard::Construct( const FArguments& InArgs )
{
	DesiredSize = InArgs._DesiredSize.Get();
	OnCanceled = InArgs._OnCanceled;
	OnFinished = InArgs._OnFinished;
	OnFirstPageBackClicked = InArgs._OnFirstPageBackClicked;
	bShowHistoryButtons = InArgs._ShowHistoryButtons;

	TSharedPtr<SVerticalBox> PageListBox;
	TSharedPtr< SUniformGridPanel > ButtonGrid; 

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(10.0f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(BreadcrumbTrail, SBreadcrumbTrail<int32>)
				.OnCrumbClicked(this, &SWizard::HandleBreadcrumbClicked)
				.Visibility(InArgs._ShowBreadcrumbs ? EVisibility::Visible : EVisibility::Collapsed)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(InArgs._ShowBreadcrumbs ? FMargin(0, 10, 0, 0) : FMargin(0))
			[
				SNew(STextBlock)
				.TextStyle(InArgs._PageTitleTextStyle)
				.Text(this, &SWizard::HandleGetPageTitle)
				.Visibility(InArgs._ShowPageTitle ? EVisibility::Visible : EVisibility::Collapsed)
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 0.0f, 20.0f, 0.0f)
				[
					SAssignNew(PageListBox, SVerticalBox)
					.Visibility(InArgs._ShowPageList ? EVisibility::Visible : EVisibility::Collapsed)
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.FillWidth(1.0f)
				[
					// widget switcher
					SAssignNew(WidgetSwitcher, SWidgetSwitcher)
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0)
			[
				InArgs._PageFooter.Widget
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(0.0f, 10.0f, 0.0f, 0.0f)
			[
				SAssignNew(ButtonGrid, SUniformGridPanel)
				.SlotPadding(FCoreStyle::Get().GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FCoreStyle::Get().GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FCoreStyle::Get().GetFloat("StandardDialog.MinDesiredSlotHeight"))
			
				+ SUniformGridPanel::Slot(0, 0)
				[
					// 'Prev' button
					SNew(SButton)
					.ButtonStyle(InArgs._ButtonStyle)
					.TextStyle(InArgs._ButtonTextStyle)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.ContentPadding(FCoreStyle::Get().GetMargin("StandardDialog.ContentPadding"))
					.IsEnabled(this, &SWizard::HandlePrevButtonIsEnabled)
					.OnClicked(this, &SWizard::HandlePrevButtonClicked)
					.Visibility(this, &SWizard::HandlePrevButtonVisibility)
					.ToolTipText(LOCTEXT("PrevButtonTooltip", "Go back to the previous step"))
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.Padding(0.0f, 0.0f)
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SImage)
							.Image(FCoreStyle::Get().GetBrush("Wizard.BackIcon"))
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.TextStyle(InArgs._ButtonTextStyle)
							.Text(LOCTEXT("PrevButtonLabel", "Back"))
						]
					]
				]

				+ SUniformGridPanel::Slot(1, 0)
				[
					// 'Next' button
					SNew(SButton)
					.ButtonStyle(InArgs._FinishButtonStyle)
					.TextStyle(InArgs._ButtonTextStyle)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.ContentPadding(FCoreStyle::Get().GetMargin("StandardDialog.ContentPadding"))
					.IsEnabled(this, &SWizard::HandleNextButtonIsEnabled)
					.OnClicked(this, &SWizard::HandleNextButtonClicked)
					.Visibility(this, &SWizard::HandleNextButtonVisibility)
					.ToolTipText(LOCTEXT("NextButtonTooltip", "Go to the next step"))
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.TextStyle(InArgs._ButtonTextStyle)
							.Text(LOCTEXT("NextButtonLabel", "Next"))
						]

						+SHorizontalBox::Slot()
						.Padding(2.0f, 0.0f)
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SImage)
							.Image(FCoreStyle::Get().GetBrush("Wizard.NextIcon"))
						]
					]
				]

				+ SUniformGridPanel::Slot(2,0)
				[
					// 'Finish' button
					SAssignNew(FinishButton, SButton)
					.ButtonStyle(InArgs._FinishButtonStyle)
					.TextStyle(InArgs._ButtonTextStyle)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.ContentPadding(FCoreStyle::Get().GetMargin("StandardDialog.ContentPadding"))
					.IsEnabled(InArgs._CanFinish)
					.OnClicked(this, &SWizard::HandleFinishButtonClicked)
					.ToolTipText(InArgs._FinishButtonToolTip)
					.Text(InArgs._FinishButtonText)
				]
			]
		]
	];

	if (InArgs._ShowCancelButton)
	{
		ButtonGrid->AddSlot(3, 0)
		[
			// cancel button
			SNew(SButton)
			.ButtonStyle(InArgs._CancelButtonStyle)
			.TextStyle(InArgs._ButtonTextStyle)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ContentPadding(FCoreStyle::Get().GetMargin("StandardDialog.ContentPadding"))
			.OnClicked(this, &SWizard::HandleCancelButtonClicked)
			.ToolTipText(LOCTEXT("CancelButtonTooltip", "Cancel this wizard"))
			.Text(LOCTEXT("CancelButtonLabel", "Cancel"))
		];
	}

	// populate wizard with pages
	for (int32 SlotIndex = 0; SlotIndex < InArgs.Slots.Num(); ++SlotIndex)
	{
		FWizardPage* Page = InArgs.Slots[SlotIndex];

		Pages.Add(Page);

		if (InArgs._ShowPageList)
		{
			PageListBox->AddSlot()
			.AutoHeight()
			[
				SNew(SCheckBox)
				.IsChecked(this, &SWizard::HandlePageButtonIsChecked, SlotIndex)
				.IsEnabled(this, &SWizard::HandlePageButtonIsEnabled, SlotIndex)
				.OnCheckStateChanged(this, &SWizard::HandlePageButtonCheckStateChanged, SlotIndex)
				.Padding(FMargin(8.0f, 4.0f, 24.0f, 4.0f))
				.Style(&FCoreStyle::Get().GetWidgetStyle< FCheckBoxStyle >("ToggleButtonCheckbox"))
				[
					Page->GetButtonContent()
				]
			];
		}

		WidgetSwitcher->AddSlot()
		[
			Page->GetPageContent()
		];
	}

	OnGetNextPageIndex = InArgs._OnGetNextPageIndex;
	OnTryToMoveToPage = InArgs._OnTryToMoveToPage;

	WidgetSwitcher->SetActiveWidgetIndex(INDEX_NONE);
	ShowPage(InArgs._InitialPageIndex.Get());
}

void SWizard::ShowPage( int32 PageIndex )
{
	int32 ActivePageIndex = WidgetSwitcher->GetActiveWidgetIndex();

	if (Pages.IsValidIndex(ActivePageIndex))
	{
		Pages[ActivePageIndex].OnLeave().ExecuteIfBound();
	}

	if (Pages.IsValidIndex(PageIndex) && Pages[PageIndex].CanShow())
	{
		WidgetSwitcher->SetActiveWidgetIndex(PageIndex);

		// show the desired page
		Pages[PageIndex].OnEnter().ExecuteIfBound();
	}
	else if (Pages.IsValidIndex(0) && Pages[0].CanShow())
	{
		WidgetSwitcher->SetActiveWidgetIndex(0);

		// attempt to fall back to first page
		Pages[0].OnEnter().ExecuteIfBound();
	}
	else
	{
		ensure(false);
		WidgetSwitcher->SetActiveWidgetIndex(INDEX_NONE);
	}
}


/* SCompoundWidget overrides
 *****************************************************************************/

FVector2D SWizard::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	if (DesiredSize.IsZero())
	{
		return SCompoundWidget::ComputeDesiredSize(LayoutScaleMultiplier);
	}

	return DesiredSize;
}


/* SWizard callbacks
 *****************************************************************************/

FReply SWizard::HandleCancelButtonClicked()
{
	OnCanceled.ExecuteIfBound();

	return FReply::Handled();
}


FReply SWizard::HandleFinishButtonClicked()
{
	OnFinished.ExecuteIfBound();

	return FReply::Handled();
}

int32 SWizard::GetNextPageIndex() const
{
	int32 ActivePage = WidgetSwitcher->GetActiveWidgetIndex();
	int32 NextPage = ActivePage + 1;

	if (OnGetNextPageIndex.IsBound())
	{
		NextPage = OnGetNextPageIndex.Execute(ActivePage);
	}

	return NextPage;
}

int32 SWizard::GetPrevPageIndex() const
{
	TArray<int32> PageHistory;
	BreadcrumbTrail->GetAllCrumbData(PageHistory);

	if (PageHistory.Num() < 1)
	{
		return INDEX_NONE;
	}

	return PageHistory[PageHistory.Num() - 1];
}

void SWizard::AdvanceToPage(int32 PageIndex)
{
	int32 CurrentPage = GetCurrentPageIndex();
	if (CurrentPage != INDEX_NONE)
	{
		BreadcrumbTrail->PushCrumb(Pages[CurrentPage].GetName(), CurrentPage);
	}

	if (OnTryToMoveToPage.IsBound())
	{
		PageIndex = OnTryToMoveToPage.Execute(PageIndex);
	}

	ShowPage(PageIndex);
}

TSharedPtr<SWidget> SWizard::GetWidgetToFocusOnActivate()
{
	return FinishButton;
}

FReply SWizard::HandleNextButtonClicked()
{
	int32 NextPage = GetNextPageIndex();
	AdvanceToPage(NextPage);

	return FReply::Handled();
}

bool SWizard::HandleNextButtonIsEnabled() const
{
	if (!bShowHistoryButtons)
	{
		return false;
	}

	int32 NextPage = GetNextPageIndex();
	return CanShowPage(NextPage);
}

EVisibility SWizard::HandleNextButtonVisibility() const
{
	if (!bShowHistoryButtons)
	{
		return EVisibility::Collapsed;
	}

	int32 NextPage = GetNextPageIndex();
	if (Pages.IsValidIndex(NextPage))
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

void SWizard::HandlePageButtonCheckStateChanged( ECheckBoxState NewState, int32 PageIndex )
{
	if (NewState == ECheckBoxState::Checked)
	{
		ShowPage(PageIndex);
	}
}

ECheckBoxState SWizard::HandlePageButtonIsChecked( int32 PageIndex ) const
{
	if (PageIndex == WidgetSwitcher->GetActiveWidgetIndex())
	{
		return ECheckBoxState::Checked;
	}

	return ECheckBoxState::Unchecked;
}

bool SWizard::HandlePageButtonIsEnabled( int32 PageIndex ) const
{
	return CanShowPage(PageIndex);
}

FReply SWizard::HandlePrevButtonClicked()
{
	if ( WidgetSwitcher->GetActiveWidgetIndex() == 0 && OnFirstPageBackClicked.IsBound() )
	{
		return OnFirstPageBackClicked.Execute();
	}

	int32 PrevPage = GetPrevPageIndex();
	ShowPage(PrevPage);
	BreadcrumbTrail->PopCrumb();

	return FReply::Handled();
}

bool SWizard::HandlePrevButtonIsEnabled() const
{
	if (bShowHistoryButtons && WidgetSwitcher->GetActiveWidgetIndex() == 0 && OnFirstPageBackClicked.IsBound())
	{
		return true;
	}

	int32 PrevPage = GetPrevPageIndex();
	return CanShowPage(PrevPage);
}

EVisibility SWizard::HandlePrevButtonVisibility() const
{
	if (!bShowHistoryButtons)
	{
		return EVisibility::Collapsed;
	}

	int32 PrevPage = GetPrevPageIndex();
	if (PrevPage != INDEX_NONE || OnFirstPageBackClicked.IsBound())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

int32 SWizard::GetNumPages() const
{
	return WidgetSwitcher->GetNumWidgets();
}

int32 SWizard::GetCurrentPageIndex() const
{
	return WidgetSwitcher->GetActiveWidgetIndex();
}

int32 SWizard::GetPageIndex(const TSharedPtr<SWidget>& PageWidget) const
{
	if (!PageWidget.IsValid())
	{
		return INDEX_NONE;
	}

	return WidgetSwitcher->GetWidgetIndex(PageWidget.ToSharedRef());
}

void SWizard::HandleBreadcrumbClicked(const int32& PageIndex)
{
	int32 CurrentCrumb = BreadcrumbTrail->PeekCrumb();
	while (CurrentCrumb != PageIndex)
	{
		BreadcrumbTrail->PopCrumb();
		CurrentCrumb = BreadcrumbTrail->PeekCrumb();
	}

	BreadcrumbTrail->PopCrumb();
	ShowPage(PageIndex);
}

FText SWizard::HandleGetPageTitle() const
{
	int32 PageIndex = GetCurrentPageIndex();
	if (Pages.IsValidIndex(PageIndex))
	{
		return Pages[PageIndex].GetName();
	}
	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
