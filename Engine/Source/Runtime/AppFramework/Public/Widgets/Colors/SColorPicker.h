// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Framework/SlateDelegates.h"

class FColorTheme;
class SBorder;
class SColorThemesViewer;
class SComboButton;
class SThemeColorBlocksBar;
enum class EColorBlockAlphaDisplayMode : uint8;
class SColorWheel;

/** Called when the color picker cancel button is pressed */
DECLARE_DELEGATE_OneParam(FOnColorPickerCancelled, FLinearColor);


/**
 * Enumerates color channels (do not reorder).
 */
enum class EColorPickerChannels
{
	Red,
	Green,
	Blue,
	Alpha,
	Hue,
	Saturation,
	Value
};


/**
 * Enumerates color picker modes.
 */
enum class EColorPickerModes
{
	Spectrum,
	Wheel
};


/**
 * Enumerates hexadecimal text modes.
 */
enum class EColorPickerHexMode
{
	SRGB,
	Linear
};


/**
 * Struct for holding individual pointers to float values.
 */
struct FColorChannels
{
	FColorChannels()
	{
		Red = Green = Blue = Alpha = nullptr;
	}

	float* Red;
	float* Green;
	float* Blue;
	float* Alpha;
};

/**
 * Class for placing a color picker. If all you need is a standalone color picker,
 * use the functions OpenColorPicker and DestroyColorPicker, since they hold a static
 * instance of the color picker.
 */
class SColorPicker
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SColorPicker)
		: _TargetColorAttribute(FLinearColor(ForceInit))
		, _ColorWheelBrush(FCoreStyle::Get().GetBrush("ColorWheel.HueValueCircle"))
		, _UseAlpha(true)
		, _OnlyRefreshOnMouseUp(false)
		, _OnlyRefreshOnOk(false)
		, _OnColorCommitted()
		, _PreColorCommitted()
		, _OnColorPickerCancelled()
		, _OnColorPickerWindowClosed()
		, _OnInteractivePickBegin()
		, _OnInteractivePickEnd()
		, _ParentWindow()
		, _DisplayGamma(2.2f)
		, _sRGBOverride()
		, _DisplayInlineVersion(false)
		, _OverrideColorPickerCreation(false)
		, _ExpandAdvancedSection(false)
		, _ClampValue(false)
		, _OptionalOwningDetailsView(nullptr)
	{ }

		/** The color that is being targeted as a TAttribute */
		SLATE_ATTRIBUTE(FLinearColor, TargetColorAttribute)

		/** ColorWheelBrush to use */
		SLATE_ATTRIBUTE(const FSlateBrush*, ColorWheelBrush)

		/** An array of color pointers this color picker targets */
		SLATE_ARGUMENT_DEPRECATED(TArray<FColor*>, TargetFColors, 5.2, "TargetFColors is deprecated. Use OnColorCommitted to get the selected color.")
		
		/** An array of linear color pointers this color picker targets */
		SLATE_ARGUMENT_DEPRECATED(TArray<FLinearColor*>, TargetLinearColors, 5.2, "TargetLinearColors is deprecated. Use OnColorCommitted to get the selected color.")

		/**
		 * An array of color pointer structs this color picker targets
		 * Only to keep compatibility with wx. Should be removed once wx is gone.
		 */
		SLATE_ARGUMENT_DEPRECATED(TArray<FColorChannels>, TargetColorChannels, 5.2, "TargetColorChannels is deprecated. Use OnColorCommitted to get the selected color.")

		/** Whether the ability to pick the alpha value is enabled */
		SLATE_ATTRIBUTE(bool, UseAlpha)

		/** Prevents immediate refreshs for performance reasons. */
		SLATE_ATTRIBUTE(bool, OnlyRefreshOnMouseUp)

		/** Prevents multiple refreshes when requested. */
		SLATE_ATTRIBUTE(bool, OnlyRefreshOnOk)

		/** The event called when the color is committed */
		SLATE_EVENT(FOnLinearColorValueChanged, OnColorCommitted)

		UE_DEPRECATED(5.2, "PreColorCommitted is deprecated. Use OnColorCommitted to update your values.")
		/** The event called before the color is committed */
		SLATE_EVENT(FOnLinearColorValueChanged, PreColorCommitted)

		/** The event called when the color picker cancel button is pressed */
		SLATE_EVENT(FOnColorPickerCancelled, OnColorPickerCancelled)

		/** The event called when the color picker parent window is closed */
		SLATE_EVENT(FOnWindowClosed, OnColorPickerWindowClosed)

		/** The event called when a slider drag, color wheel drag or dropper grab starts */
		SLATE_EVENT(FSimpleDelegate, OnInteractivePickBegin)

		/** The event called when a slider drag, color wheel drag or dropper grab finishes */
		SLATE_EVENT(FSimpleDelegate, OnInteractivePickEnd)

		/** A pointer to the parent window */
		SLATE_ATTRIBUTE(TSharedPtr<SWindow>, ParentWindow)

		/** Sets the display Gamma setting - used to correct colors sampled from the screen */
		SLATE_ATTRIBUTE(float, DisplayGamma)

		/** Overrides the checkbox value of the sRGB option. */
		SLATE_ARGUMENT(TOptional<bool>, sRGBOverride)

		/** If true, this color picker will be a stripped down version of the full color picker */
		SLATE_ARGUMENT(bool, DisplayInlineVersion)

		/** If true, this color picker will have non-standard creation behavior */
		SLATE_ARGUMENT(bool, OverrideColorPickerCreation)

		/** If true, the Advanced section will be expanded, regardless of the remembered state */
		SLATE_ARGUMENT(bool, ExpandAdvancedSection)

		/** The LinearColor is expected to be converted to a FColor, set the clamp channel value. */
		SLATE_ARGUMENT(bool, ClampValue)

		/** Allows a details view to own the color picker so refreshing another details view doesn't close it */
		SLATE_ATTRIBUTE(TSharedPtr<SWidget>, OptionalOwningDetailsView)

	SLATE_END_ARGS()
	
	/** A default window size for the color picker which looks nice */
	static APPFRAMEWORK_API const FVector2D DEFAULT_WINDOW_SIZE;

	/**	Destructor. */
	APPFRAMEWORK_API ~SColorPicker();

public:

	/**
	 * Construct the widget
	 *
	 * @param InArgs Declaration from which to construct the widget.
	 */
	APPFRAMEWORK_API void Construct(const FArguments& InArgs);

	/** Gets the (optionally set) owning details view of the current color picker */
	TSharedPtr<SWidget> GetOptionalOwningDetailsView()
	{
		if (OptionalOwningDetailsView.IsValid())
		{
			return OptionalOwningDetailsView.Pin();
		}
		else
		{
			return nullptr;
		}
	}

	/** Delegate to override color picker creation behavior */
	DECLARE_DELEGATE_OneParam(FOnColorPickerCreationOverride, const TSharedRef<SColorPicker>&);
	static APPFRAMEWORK_API FOnColorPickerCreationOverride OnColorPickerNonModalCreateOverride;

	/** Delegate to override color picker destruction behavior */
	DECLARE_DELEGATE(FOnColorPickerDestructionOverride);
	static APPFRAMEWORK_API FOnColorPickerDestructionOverride OnColorPickerDestroyOverride;

	/** Eye dropper button creation args */
	struct FEyeDropperButtonArgs
	{
		/** Invoked when a new value is selected by the dropper */
		TDelegate<void(FLinearColor)> OnValueChanged;

		/** Invoked when the dropper goes from inactive to active */
		FSimpleDelegate OnBegin;

		/** Invoked when the dropper goes from active to inactive */
		TDelegate<void(bool)> OnComplete;

		/** Sets the display Gamma setting - used to correct colors sampled from the screen */
		TAttribute<float> DisplayGamma = TAttribute<float>();
	};

	/** Creates a standalone eye dropper button that functions identically to the button within the color picker */
	static APPFRAMEWORK_API TSharedRef<SButton> CreateStandaloneEyeDropperButton(const FEyeDropperButtonArgs& InArgs);

protected:

	/** Backup all the colors that are being modified */
	APPFRAMEWORK_API void BackupColors();

	APPFRAMEWORK_API bool ApplyNewTargetColor(bool bForceUpdate = false);

	APPFRAMEWORK_API void GenerateDefaultColorPickerContent(bool bAdvancedSectionExpanded);
	APPFRAMEWORK_API void GenerateInlineColorPickerContent();

	FLinearColor GetCurrentColor() const
	{
		return CurrentColorHSV;
	}

	/** Calls the user defined delegate for when the color changes are discarded */
	APPFRAMEWORK_API void DiscardColor();

	/** Sets new color in ether RGB or HSV */
	APPFRAMEWORK_API bool SetNewTargetColorRGB(const FLinearColor& NewValue, bool bForceUpdate = false);
	APPFRAMEWORK_API bool SetNewTargetColorHSV(const FLinearColor& NewValue, bool bForceUpdate = false);

	APPFRAMEWORK_API void UpdateColorPick();
	APPFRAMEWORK_API void UpdateColorPickMouseUp();

	APPFRAMEWORK_API void BeginAnimation(FLinearColor Start, FLinearColor End);

	UE_DEPRECATED(5.6, "HideSmallTrash is deprecated. The delete button for theme colors is managed by the SThemeColorBlocksBar widget.")
	APPFRAMEWORK_API void HideSmallTrash();

	UE_DEPRECATED(5.6, "ShowSmallTrash is deprecated. The delete button for theme colors is managed by the SThemeColorBlocksBar widget.")
	APPFRAMEWORK_API void ShowSmallTrash();

	/** Cycles the color picker's mode. */
	APPFRAMEWORK_API void CycleMode();

	/**
	 * Creates a color slider widget for the specified channel.
	 *
	 * @param Channel The color channel to create the widget for.
	 * @return The new slider.
	 */
	APPFRAMEWORK_API TSharedRef<SWidget> MakeColorSlider(EColorPickerChannels Channel) const;

	/**
	 * Creates a color spin box widget for the specified channel.
	 *
	 * @param Channel The color channel to create the widget for.
	 * @return The new spin box.
	 */
	APPFRAMEWORK_API TSharedRef<SWidget> MakeColorSpinBox(EColorPickerChannels Channel) const;

	/**
	 * Creates the color preview box widget.
	 *
	 * @return The new color preview box.
	 */
	APPFRAMEWORK_API TSharedRef<SWidget> MakeColorPreviewBox() const;

private:

	// Callback for the active timer to animate the color post-construct
	APPFRAMEWORK_API EActiveTimerReturnType AnimatePostConstruct(double InCurrentTime, float InDeltaTime);

	// Callback for getting the end color of a color spin box gradient.
	APPFRAMEWORK_API FLinearColor GetGradientEndColor(EColorPickerChannels Channel) const;

	// Callback for getting the start color of a color spin box gradient.
	APPFRAMEWORK_API FLinearColor GetGradientStartColor(EColorPickerChannels Channel) const;

	// Callback for getting the array of colors for a color spin box gradient.
	APPFRAMEWORK_API TArray<FLinearColor> GetGradientColors(EColorPickerChannels Channel) const;

	// Callback for getting the alpha display mode of the color preview swatches.
	APPFRAMEWORK_API EColorBlockAlphaDisplayMode HandleColorPreviewAlphaMode() const;

	// Callback for clicking the Cancel button.
	APPFRAMEWORK_API FReply HandleCancelButtonClicked();

	// Callback for pressing a mouse button in the color area.
	APPFRAMEWORK_API FReply HandleColorAreaMouseDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	// Callback for clicking the color picker mode button.
	APPFRAMEWORK_API FReply HandleColorPickerModeButtonClicked();

	// Callback for getting the visibility of the given color picker mode.
	APPFRAMEWORK_API EVisibility HandleColorPickerModeVisibility(EColorPickerModes Mode) const;

	// Callback for setting the visibility of the themes panel.
	APPFRAMEWORK_API FReply ToggleThemePanelVisibility();

	// Callback for getting the icon to display on the Show/Hide themes panel button.
	APPFRAMEWORK_API const FSlateBrush* HandleThemePanelButtonImageBrush() const;

	// Callback for getting the visibility of the themes panel.
	APPFRAMEWORK_API EVisibility HandleThemesPanelVisibility() const;

	// Callback for getting the tooltip for the color theme panel
	APPFRAMEWORK_API FText GetColorThemePanelToolTipText() const;

	// Callback for getting the end color of a color slider.
	APPFRAMEWORK_API FLinearColor HandleColorSliderEndColor(EColorPickerChannels Channel) const;

	// Callback for getting the start color of a color slider.
	APPFRAMEWORK_API FLinearColor HandleColorSliderStartColor(EColorPickerChannels Channel) const;

	// Callback for value changes in the color wheel picker.
	APPFRAMEWORK_API void HandleColorWheelValueChanged(FLinearColor NewValue);

	// Callback for value changes in the color spectrum picker.
	APPFRAMEWORK_API void HandleColorSpectrumValueChanged(FLinearColor NewValue);

	// Callback for getting the value of a color spin box.
	APPFRAMEWORK_API float HandleColorSpinBoxValue(EColorPickerChannels Channel) const;

	// Callback for value changes in a color spin box.
	APPFRAMEWORK_API void HandleColorSpinBoxValueChanged(float NewValue, EColorPickerChannels Channel);

	// Callback for start of eye dropper interactions.
	APPFRAMEWORK_API void HandleEyeDropperButtonBegin();

	// Callback for completed eye dropper interactions.
	APPFRAMEWORK_API void HandleEyeDropperButtonComplete(bool bCancelled);

	// Callback for getting the text in the hex linear box.
	APPFRAMEWORK_API FText HandleHexBoxText() const;

	// Callback for committed text in the hex input box (sRGB gamma).
	APPFRAMEWORK_API void HandleHexInputTextCommitted(const FText& Text, ETextCommit::Type CommitType);

	// Callback for changing the HSV value of the current color.
	APPFRAMEWORK_API void HandleHSVColorChanged(FLinearColor NewValue);

	// Callback for when interactive user input begins.
	APPFRAMEWORK_API void HandleInteractiveChangeBegin();

	// Callback for when interactive user input ends.
	APPFRAMEWORK_API void HandleInteractiveChangeEnd();

	// Callback for when interactive user input ends.
	APPFRAMEWORK_API void HandleInteractiveChangeEnd(float NewValue);

	// Callback for clicking the new color preview block.
	APPFRAMEWORK_API FReply HandleNewColorPreviewClicked();

	// Callback for clicking the old color preview block.
	APPFRAMEWORK_API FReply HandleOldColorPreviewClicked();

	// Callback for getting the visibility of the old/new color buttons
	APPFRAMEWORK_API EVisibility HandleColorPreviewButtonVisibility() const;

	// Callback for setting the visibility of the new color preview button image
	APPFRAMEWORK_API void SetNewColorPreviewImageVisibility(EVisibility InButtonVisibility);

	// Callback for setting the visibility of the old color preview button image
	APPFRAMEWORK_API void SetOldColorPreviewImageVisibility(EVisibility InButtonVisibility);

	// Callback for getting the visibility of the new color preview button image
	APPFRAMEWORK_API EVisibility GetNewColorPreviewImageVisibility() const;

	// Callback for getting the visibility of the old color preview button image
	APPFRAMEWORK_API EVisibility GetOldColorPreviewImageVisibility() const;

	// Callback for clicking the OK button.
	APPFRAMEWORK_API FReply HandleOkButtonClicked();

	// Callback for checking whether sRGB colors should be rendered.
	APPFRAMEWORK_API bool HandleColorPickerUseSRGB() const;

	// Callback for when the parent window has been closed.
	APPFRAMEWORK_API void HandleParentWindowClosed(const TSharedRef<SWindow>& Window);

	// Callback for changing the RGB value of the current color.
	APPFRAMEWORK_API void HandleRGBColorChanged(FLinearColor NewValue);

	// Callback for changing the checked state of the sRGB check box.
	APPFRAMEWORK_API void HandleSRGBCheckBoxCheckStateChanged(ECheckBoxState InIsChecked);

	// Callback for determining whether the sRGB check box should be checked.
	APPFRAMEWORK_API ECheckBoxState HandleSRGBCheckBoxIsChecked() const;

	// Callback for selecting a color in the color theme bar.
	APPFRAMEWORK_API void HandleThemeBarColorSelected(FLinearColor NewValue);

	// Build the menu widget for selecting the hexadecimal text mode
	APPFRAMEWORK_API TSharedRef<SWidget> MakeHexModeMenu();

	// Get the text to display on the hexadecimal text mode button
	APPFRAMEWORK_API FText HandleHexModeButtonText() const;

	// Callback for when the hexadecimal text mode changes
	APPFRAMEWORK_API void OnHexModeSelected(EColorPickerHexMode InHexMode);

private:
	
	/** The color that is being targeted as a TAttribute */
	TAttribute<FLinearColor> TargetColorAttribute;

	/** The brush used as Color wheel as a TAttribute */
	TAttribute<const FSlateBrush*> ColorWheelBrush;

	/** The current color being picked in HSV */
	FLinearColor CurrentColorHSV;

	/** The current color being picked in RGB */
	FLinearColor CurrentColorRGB;
	
	/** The old color to be changed in HSV */
	FLinearColor OldColor;
	
	/** Color end point to animate to */
	FLinearColor ColorEnd;

	/** Color start point to animate from */
	FLinearColor ColorBegin;

	/** Holds the color picker's mode. */
	EColorPickerModes CurrentMode;

	/** Holds the color picker's hex mode. */
	EColorPickerHexMode HexMode;

	/** Time, used for color animation */
	float CurrentTime;

	/** The max time allowed for updating before we shut off auto-updating */
	static APPFRAMEWORK_API const double MAX_ALLOWED_UPDATE_TIME;

	/** If true, then the performance is too bad to have auto-updating */
	bool bPerfIsTooSlowToUpdate;

	/** Whether or not the color uses Alpha or not */
	TAttribute<bool> bUseAlpha;

	/** Whether or not to display color as sRGB */
	bool bUseSRGB = false;

	/** Prevents immediate refresh for performance reasons. */
	bool bOnlyRefreshOnMouseUp;

	/** Prevents multiple refreshes when requested. */
	bool bOnlyRefreshOnOk;

	/** true if the picker was closed via the OK or Cancel buttons, false otherwise */
	bool bClosedViaOkOrCancel;

	/** Pointer to the parent window, so we know to destroy it when done */
	TWeakPtr<SWindow> ParentWindowPtr;

	/** The widget which holds the currently selected theme */
	TSharedPtr<SThemeColorBlocksBar> CurrentThemeBar;

	/** Visibility of the new color preview button image (visible when hovered) */
	EVisibility NewColorPreviewImageVisibility;

	/** Visibility of the old color preview button image (visible when hovered) */
	EVisibility OldColorPreviewImageVisibility;

	/** Sets the display Gamma setting - used to correct colors sampled from the screen */
	TAttribute<float> DisplayGamma;

	/** True if this color picker is an inline color picker */
	bool bColorPickerIsInlineVersion;

	/** True if something has overridden the color picker's creation behavior */
	bool bColorPickerCreationIsOverridden;

	/** Tracks whether the user is moving a value spin box, the color wheel and the dropper */
	bool bIsInteractive;

	/** Is true if the color picker creation behavior can be overridden */
	bool bValidCreationOverrideExists;
	
	/** The current display gamma used to correct colors picked from the display. */
	bool bClampValue;

	/** Whether or not the color themes panel is visible or collapsed */
	bool bIsThemePanelVisible;

private:

	/** Invoked when a new value is selected on the color wheel */
	FOnLinearColorValueChanged OnColorCommitted;

	/** Invoked when the color picker cancel button is pressed */
	FOnColorPickerCancelled OnColorPickerCancelled;

	/** Invoked when a slider drag, color wheel drag or dropper grab starts */
	FSimpleDelegate OnInteractivePickBegin;

	/** Invoked when a slider drag, color wheel drag or dropper grab finishes */
	FSimpleDelegate OnInteractivePickEnd;

	/** Invoked when the color picker window closes. */
	FOnWindowClosed OnColorPickerWindowClosed;

	/** Allows a details view to own the color picker so refreshing another details view doesn't close it */
	TWeakPtr<SWidget> OptionalOwningDetailsView;

	/** The Color Wheel widget embedded in this picker */
	TSharedPtr<SColorWheel> ColorWheel;
};


struct FColorPickerArgs
{
	/** Whether or not the new color picker is modal. */
	bool bIsModal = false;

	/** The parent for the new color picker window */
	TSharedPtr<SWidget> ParentWidget;

	/** Whether or not to enable the alpha slider. */
	bool bUseAlpha = false;

	/** Whether to disable the refresh except on mouse up for performance reasons. */
	bool bOnlyRefreshOnMouseUp = false;

	/** Whether to disable the refresh until the picker closes. */
	bool bOnlyRefreshOnOk = false;

	/** Whether to automatically expand the Advanced section. */
	bool bExpandAdvancedSection = true;
	
	/** Whether to open the color picker as a menu window. */
	bool bOpenAsMenu = false;

	/** Set true if values should be max to 1.0. HDR values may go over 1.0. */
	bool bClampValue = false;

	/** The current display gamma used to correct colors picked from the display. */
	TAttribute<float> DisplayGamma = 2.2f;

	/** If set overrides the global option for the desired setting of sRGB mode. */
	TOptional<bool> sRGBOverride = TOptional<bool>();

	UE_DEPRECATED(5.2, "ColorArray is deprecated. Use OnColorCommitted to update your values.")
	/** An array of FColors to target. */
	const TArray<FColor*>* ColorArray = nullptr;

	UE_DEPRECATED(5.2, "LinearColorArray is deprecated. Use OnColorCommitted to update your values.")
	/** An array of FLinearColors to target. */
	const TArray<FLinearColor*>* LinearColorArray = nullptr;

	UE_DEPRECATED(5.2, "ColorChannelsArray is deprecated. Use OnColorCommitted to update your values.")
	/** An array of FColorChannels to target. (deprecated now that wx is gone?) */
	const TArray<FColorChannels>* ColorChannelsArray = nullptr;

	/** A delegate to be called when the color changes. */
	FOnLinearColorValueChanged OnColorCommitted;

	UE_DEPRECATED(5.2, "PreColorCommitted is deprecated. Use OnColorCommitted to update your values.")
	/** A delegate to be called before the color change is committed. */
	FOnLinearColorValueChanged PreColorCommitted;

	/** A delegate to be called when the color picker window closes. */
	FOnWindowClosed OnColorPickerWindowClosed;

	/** A delegate to be called when the color picker cancel button is pressed */
	FOnColorPickerCancelled OnColorPickerCancelled;

	/** A delegate to be called when a slider drag, color wheel drag or dropper grab starts */
	FSimpleDelegate OnInteractivePickBegin;

	/** A delegate to be called when a slider drag, color wheel drag or dropper grab finishes */
	FSimpleDelegate OnInteractivePickEnd;

	UE_DEPRECATED(5.2, "InitialColorOverride is deprecated. Use InitialColor to set the initial color.")
	/** Overrides the initial color set on the color picker. */
	FLinearColor InitialColorOverride = FLinearColor::White;

	/** The initial color set on the color picker. */
	FLinearColor InitialColor = FLinearColor::White;

	/** Allows a details view to own the color picker so refreshing another details view doesn't close it */
	TSharedPtr<SWidget> OptionalOwningDetailsView;

	/** Default constructor. */
	FColorPickerArgs() = default;

	FColorPickerArgs(FLinearColor InInitialColor, FOnLinearColorValueChanged InOnColorCommitted)
		: OnColorCommitted(MoveTemp(InOnColorCommitted))
		, InitialColor(InInitialColor)
	{}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FColorPickerArgs(const FColorPickerArgs&) = default;
	FColorPickerArgs(FColorPickerArgs&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

/** Get a pointer to the static color picker, or nullptr if it does not exist. */
APPFRAMEWORK_API TSharedPtr<SColorPicker> GetColorPicker();

/** Open up the static color picker, destroying any previously existing one. */
APPFRAMEWORK_API bool OpenColorPicker(const FColorPickerArgs& Args);

/**
 * Destroy the current color picker. Necessary if the values the color picker
 * currently targets become invalid.
 */
APPFRAMEWORK_API void DestroyColorPicker();
