// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Layout/Children.h"
#include "Framework/MultiBox/MultiBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Colors/SColorBlock.h"

class FArrangedChildren;
class SEditableTextBox;
class SErrorText;
class STextBlock;
class SThemeColorBlocksBar;
class UToolMenu;

struct FColorInfo
{
	TSharedPtr<FLinearColor> Color;

	FText Label;

	FColorInfo(TSharedPtr<FLinearColor> InColor)
	{
		Color = InColor;
		Label = FText();
	}

	FColorInfo(TSharedPtr<FLinearColor> InColor, FText InLabel)
	{
		Color = InColor;
		Label = InLabel;
	}
};

/**
 * A Color Theme is a name and an array of Colors.
 * It also holds and array of refresh callbacks which it calls every time it changes at all.
 */
class FColorTheme
{
public:

	FColorTheme(const FString& InName = TEXT(""), const TArray< TSharedPtr<FColorInfo> >& InColors = TArray< TSharedPtr<FColorInfo> >());

	/** Get a list of all the colors in the theme */
	const TArray< TSharedPtr<FColorInfo> >& GetColors() const
	{
		return Colors;
	}

	/** Insert a color at a specific point in the list and broadcast change */
	void InsertNewColor(TSharedPtr<FLinearColor> InColor, int32 InsertPosition);
	void InsertNewColor(TSharedPtr<FColorInfo> InColor, int32 InsertPosition);

	/** Check to see if a color is already present in the list */
	int32 FindApproxColor(const FLinearColor& InColor, float Tolerance = KINDA_SMALL_NUMBER) const;

	/** Remove all colors from the list, broadcast change */
	void RemoveAll();

	/** Remove specific color from the list, broadcast change */
	int32 RemoveColor(const TSharedPtr<FLinearColor> InColor);
	
	FString Name;

	DECLARE_EVENT( FColorTheme, FRefreshEvent );
	FRefreshEvent& OnRefresh()
	{
		return RefreshEvent;
	}

private:

	TArray< TSharedPtr<FColorInfo> > Colors;

	FRefreshEvent RefreshEvent;
};


/**
 * The SColorTrash is a multipurpose widget which allows FColorDragDrops
 * to be dropped on to to be 
 */
class UE_DEPRECATED(5.6, "SColorTrash is deprecated. SThemeColorBlocksBar now uses a simple SButton for its delete button.") SColorTrash
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SColorTrash)
		: _UsesSmallIcon(false)
	{ }

		SLATE_ATTRIBUTE(bool, UsesSmallIcon)
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs );

protected:	

	/**
	 * Called during drag and drop when the drag enters a widget.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 * @return A reply that indicated whether this event was handled.
	 */
	virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;

	/**
	 * Called during drag and drop when the drag leaves a widget.
	 *
	 * @param DragDropEvent   The drag and drop event.
	 * @return A reply that indicated whether this event was handled.
	 */
	virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;

	/**
	 * Called when the user is dropping something onto a widget; terminates drag and drop.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 * @return A reply that indicated whether this event was handled.
	 */
	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	
	const FSlateBrush* GetBorderStyle() const;

private:

	/** Determines whether to draw the border to show activation */
	bool bBorderActivated;
};


/**
 * SThemeColorBlocks are Color Blocks which point to a Color in a ColorTheme.
 * They can be dragged and dropped, and clicking on one in the Color Picker will
 * give the color that they point to.
 */
class SThemeColorBlock
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SThemeColorBlock)
		: _Color()
		, _ColorInfo()
		, _OnSelectColor()
		, _Parent()
		, _ShowTrashCallback()
		, _HideTrashCallback()
		, _UseSRGB()
		, _UseAlpha()
		, _SupportsDrag(true)
	{ }

		/** A pointer to the color this block uses */
		SLATE_ATTRIBUTE(TSharedPtr<FLinearColor>, Color)

		/** The info for the color this block uses */
		SLATE_ATTRIBUTE(TSharedPtr<FColorInfo>, ColorInfo)
		
		/** Event called when this block is clicked */
		SLATE_EVENT(FOnLinearColorValueChanged, OnSelectColor)

		/** A pointer to the theme color blocks bar that is this block's origin */
		SLATE_ATTRIBUTE(TSharedPtr<SThemeColorBlocksBar>, Parent)
		
		/** Callback to pass down to the FColorDragDrop for it to show the trash */
		SLATE_EVENT(FSimpleDelegate, ShowTrashCallback)
		
		/** Callback to pass down to the FColorDragDrop for it to hide the trash */
		SLATE_EVENT(FSimpleDelegate, HideTrashCallback)
		
		/** Whether to display sRGB color */
		SLATE_ATTRIBUTE(bool, UseSRGB)

		/** Whether the ability to pick the alpha value is enabled */
		SLATE_ATTRIBUTE(bool, UseAlpha)

		/** Whether the color block supports drag/drop operations */
		SLATE_ATTRIBUTE(bool, SupportsDrag)

	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 *
	 * @param InArgs   Declaration from which to construct the widget.
	 */
	void Construct(const FArguments& InArgs );
	
	
private:
	void OnColorBlockRename();
	FText GetLabel() const;
	void SetLabel(const FText& NewColorLabel, ETextCommit::Type CommitInfo);
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnDragDetected( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

	FLinearColor GetColor() const;

	FText GetRedText() const;
	FText GetGreenText() const;
	FText GetBlueText() const;
	FText GetAlphaText() const;
	FText GetHueText() const;
	FText GetSaturationText() const;
	FText GetValueText() const;

	/**
	 * Function for formatting text for the tooltip which has limited space.
	 */
	FText FormatToolTipText(const FText& ColorIdentifier, float Value) const;
	
	EColorBlockAlphaDisplayMode OnGetAlphaDisplayMode() const;
	bool OnReadShowBackgroundForAlpha() const;
	EVisibility OnGetAlphaVisibility() const;

	EVisibility OnGetLabelVisibility() const;

	/** A pointer to the color this block uses */
	TWeakPtr<FLinearColor> ColorPtr;

	/** The info for this color block */
	TSharedPtr<FColorInfo> ColorInfo;
	
	/** A pointer to the theme color blocks bar that is this block's origin */
	TWeakPtr<SThemeColorBlocksBar> ParentPtr;
	
	/** Event called when this block is clicked */
	FOnLinearColorValueChanged OnSelectColor;

	/** Callback to pass down to the FColorDragDrop for it to show the trash */
	FSimpleDelegate ShowTrashCallback;

	/** Callback to pass down to the FColorDragDrop for it to hide the trash */
	FSimpleDelegate HideTrashCallback;
	
	/** Whether to use display sRGB color */
	TAttribute<bool> bUseSRGB;

	/** Whether or not the color uses Alpha or not */
	TAttribute<bool> bUseAlpha;

	/** Whether the color block supports drag/drop operations */
	TAttribute<bool> bSupportsDrag;
};


DECLARE_DELEGATE_OneParam(FOnCurrentThemeChanged, TSharedPtr<FColorTheme>)


/**
 * SColorThemeBars include a ThemeColorBlocksBar in addition to a label.
 * Clicking on one will select it and set the currently used color theme to it
 */
class UE_DEPRECATED(5.6, "SColorThemeBar is deprecated. SColorThemesViewer now displays a standard menu with a list of available themes.") SColorThemeBar
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SColorThemeBar)
		: _ColorTheme()
		, _OnCurrentThemeChanged()
		, _ShowTrashCallback()
		, _HideTrashCallback()
		, _UseSRGB()
		, _UseAlpha()
	{ }

		/** The color theme that this bar is displaying */
		SLATE_ATTRIBUTE(TSharedPtr<FColorTheme>, ColorTheme)

		/** Event to be called when the current theme changes */
		SLATE_EVENT(FOnCurrentThemeChanged, OnCurrentThemeChanged)

		/** Callback to pass down to the FColorDragDrop for it to show the trash */
		SLATE_EVENT(FSimpleDelegate, ShowTrashCallback)
		/** Callback to pass down to the FColorDragDrop for it to hide the trash */
		SLATE_EVENT(FSimpleDelegate, HideTrashCallback)
		
		/** Whether to display sRGB color */
		SLATE_ATTRIBUTE(bool, UseSRGB)

		/** Whether the ability to pick the alpha value is enabled */
		SLATE_ATTRIBUTE(bool, UseAlpha)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	
	/**
	 * The system calls this method to notify the widget that a mouse button was pressed within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent );
	
private:

	FText GetThemeName() const;

	/** Text Block which shows the Color Theme's name */
	TSharedPtr<STextBlock> ThemeNameText;

	/** Color Theme that this bar is displaying */
	TWeakPtr<FColorTheme> ColorTheme;

	/** Callback to execute when the global current theme has changed */
	FOnCurrentThemeChanged OnCurrentThemeChanged;

	/** Callback to pass down to the FColorDragDrop for it to show the trash */
	FSimpleDelegate ShowTrashCallback;

	/** Callback to pass down to the FColorDragDrop for it to hide the trash */
	FSimpleDelegate HideTrashCallback;
	
	/** Whether to use display sRGB color */
	TAttribute<bool> bUseSRGB;

	/** Whether or not the color uses Alpha or not */
	TAttribute<bool> bUseAlpha;
};


/**
 * The widget that manages the dropdown menu in the Color Themes Panel
 */
class SColorThemesViewer : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SColorThemesViewer)
	{ }

		SLATE_ATTRIBUTE_DEPRECATED(bool, UseAlpha, 5.6, "UseAlpha is deprecated. Set the UseAlpha attribute of SThemeColorBlocksBar to control the opacity of theme colors.")

	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);
	
	/** Gets the current color theme */
	TSharedPtr<FColorTheme> GetCurrentColorTheme() const;

	/** Gets the recents color theme */
	TSharedPtr<FColorTheme> GetRecents() const;

	UE_DEPRECATED(5.6, "SetUseAlpha is deprecated. Set the UseAlpha attribute of SThemeColorBlocksBar to control the opacity of theme colors.")
	void SetUseAlpha(const TAttribute<bool>& InUseAlpha);

	/** Load the color theme settings from the config */
	static void LoadColorThemesFromIni();

	/** Save the color theme settings to the config */
	static void SaveColorThemesToIni();
	
	/** Callbacks to execute whenever we change the global current theme */
	DECLARE_EVENT( SColorThemesViewer, FCurrentThemeChangedEvent );
	FCurrentThemeChangedEvent& OnCurrentThemeChanged()
	{
		return CurrentThemeChangedEvent;
	}

	UE_DEPRECATED(5.6, "bSRGBEnabled is deprecated. Set the UseSRGB attribute of SThemeColorBlocksBar to control the sRGB display of theme colors.")
	static bool bSRGBEnabled;

	UE_DEPRECATED(5.6, "MenuToStandardNoReturn is deprecated. SColorThemesViewer completely manages its own ToolsMenu.")
	void MenuToStandardNoReturn();

	/** Returns true if the currently chosen theme is the "Recents" theme */
	bool IsRecentsThemeActive() const;

private:
	/** Generate the content for the dropdown menu */
	void BuildMenu(FMenuBuilder& MenuBuilder);

	/** Initiate a rename interaction */
	void StartRename();

	/** Terminate a rename interaction */
	void StopRename();

	/** Redraw the dropdown menu widget to update while keeping the menu open */
	void RefreshMenuWidget();

	void NewColorTheme();
	void DuplicateColorTheme();
	void DeleteColorTheme();

	/** Rename theme if the user has pressed Enter in the RenameTextBox */
	void CommitThemeName(const FText& InText, ETextCommit::Type InCommitType);
	void UpdateThemeNameFromTextBox();
	
	/** Sets the current color theme to the existing theme */
	void SetCurrentColorTheme(TSharedPtr<FColorTheme> NewTheme);

	/** Callback when the dropdown menu is opened/closed */
	void OnMenuOpenChanged(bool bIsOpen);

	const FSlateBrush* GetComboButtonImage() const;

	/** Gets the default color theme, optionally creates it if not present */
	static TSharedPtr<FColorTheme> GetDefaultColorTheme(bool bCreateNew = false);
	/** Gets the color theme, creates it if not present */
	static TSharedPtr<FColorTheme> GetColorTheme(const FString& ThemeName);
	/** Checks to see if this is a color theme, returns success */
	static TSharedPtr<FColorTheme> IsColorTheme(const FString& ThemeName);
	/** Makes the passed theme name unique so it doesn't clash with pre-existing themes */
	static FString MakeUniqueThemeName(const FString& ThemeName);
	/** Creates a new theme, ensuring the name is unique */
	static TSharedPtr<FColorTheme> NewColorTheme(const FString& ThemeName, const TArray< TSharedPtr<FColorInfo> >& ThemeColors = TArray< TSharedPtr<FColorInfo> >());

private:
	/** The MultiBox widget of the combo button */
	TSharedPtr<SMultiBoxWidget> MultiBoxWidget;

	/** The text box for renaming themes */
	TSharedPtr<SEditableTextBox> RenameTextBox;

	/** Callbacks to execute whenever we change the global current theme */
	FCurrentThemeChangedEvent CurrentThemeChangedEvent;

	/** Whether or not the color uses Alpha or not */
	TAttribute<bool> bUseAlpha;

	/** A static holder of the color themes for the entire program */
	static TArray<TSharedPtr<FColorTheme>> ColorThemes;

	/** A static color theme comprised of the most recently accepted color values */
	static TSharedPtr<FColorTheme> Recents;

	/** A static pointer to the color theme that is currently selected for the entire program */
	static TWeakPtr<FColorTheme> CurrentlySelectedThemePtr;
};


/**
 * A panel for displaying SColorBlocks in a FColorTheme
 */
class SThemeColorBlocksBar : public SPanel
{
public:

	/** Delegate for hooking up to an inline editable text block 'IsSelected' check. */
	DECLARE_DELEGATE_RetVal(FLinearColor, FOnGetActiveColor);

	SLATE_BEGIN_ARGS(SThemeColorBlocksBar)
		: _OnSelectColor()
		, _OnGetActiveColor()
		, _UseSRGB()
		, _UseAlpha()
		{ }

		/** A pointer to the color theme that this bar should display */
		SLATE_ATTRIBUTE_DEPRECATED(TSharedPtr<FColorTheme>, ColorTheme, 5.6, "ColorTheme Attribute is deprecated. This widget owns a SColorThemesViewer which supplies the current color theme.")

		/** Event called when a color block is clicked */
		SLATE_EVENT(FOnLinearColorValueChanged, OnSelectColor)

		/** Event called to get the current color in the color picker window */
		SLATE_EVENT(FOnGetActiveColor, OnGetActiveColor)

		UE_DEPRECATED(5.6, "ShowTrashCallback is deprecated. Visibility of the delete button is managed by this widget internally.")
		SLATE_EVENT(FSimpleDelegate, ShowTrashCallback)

		UE_DEPRECATED(5.6, "HideTrashCallback is deprecated. Visibility of the delete button is managed by this widget internally.")
		SLATE_EVENT(FSimpleDelegate, HideTrashCallback)

		/** Specify what the bar should display when no colors are present */
		SLATE_ARGUMENT_DEPRECATED(FText, EmptyText, 5.6, "EmptyText is deprecated. This widget no longer displays any text when no themes have been created.")

		/** Whether to display sRGB color */
		SLATE_ATTRIBUTE(bool, UseSRGB)

		/** Whether the ability to pick the alpha value is enabled */
		SLATE_ATTRIBUTE(bool, UseAlpha)

	SLATE_END_ARGS()

	SThemeColorBlocksBar();

	void Construct(const FArguments& InArgs);

	// Begin SWidget overrides
	virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual FChildren* GetChildren() override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	// End SWidget overrides

	/**
	 * Adds a new color block to the Bar
	 * @param Color				The	color the new block will be
	 * @param InsertPosition	The location the block will be at
	 */
	void AddNewColorBlock(FLinearColor Color, int32 InsertPosition, bool bAllowRepeat = false);

	/** Adds a new color block to the Recents color theme */
	void AddToRecents(FLinearColor Color);

	/** Returns true if the currently selected theme is Recents */
	bool IsRecentsThemeActive() const;

	/**
	 * @param ColorToRemove		The pointer to the color that should be removed
	 *
	 * @return					The index of the removed color block, INDEX_NONE if it can't be found
	 */
	int32 RemoveColorBlock(TSharedPtr<FLinearColor> ColorToRemove);

	UE_DEPRECATED(5.6, "RemoveRefreshCallback is deprecated. Color theme changes are handled by this widget internally")
	void RemoveRefreshCallback();

	UE_DEPRECATED(5.6, "RemoveRefreshCallback is deprecated. Color theme changes are handled by this widget internally")
	void AddRefreshCallback();

	/**
	 * Rebuilds the entire bar, regenerating all the constituent color blocks
	 */
	void Refresh();

	UE_DEPRECATED(5.6, "SetPlaceholderGrabOffset is deprecated. The Placeholder position is managed by this widget internally.")
	void SetPlaceholderGrabOffset(FVector2D GrabOffset);

	/** Make the delete button visible and hide the add button */
	void ShowDeleteButton();

	/** Hide the delete button and make the add button visible */
	void HideDeleteButton();

private:
	/** Callback when the add button is clicked to add the current color picker color to the current theme */
	FReply OnAddButtonClicked();

	/** Callback when the user changes the currently selected theme, which prompts this widget to update the color blocks being displayed */
	void OnThemeChanged();

	/** Get the visibility of the add button (which is hidden if the delete button is visible) */
	EVisibility GetAddButtonVisibility() const;

	/** Get the visibility of the delete button (which is hidden unless the user is dragging a color block) */
	EVisibility GetDeleteButtonVisibility() const;

	/** The children blocks of this panel */
	TSlotlessChildren<SWidget> Children;

	/** Array of color blocks belonging to the currently selected color theme */
	TArray<TSharedPtr<SThemeColorBlock>> ColorBlocks;

	/**
	 * The Color Theme that this SThemeColorBlockBar is displaying.
	 * This is a TAttribute so it can re-get the theme when it changes rather than rely on a delegate to refresh it
	 */
	TSharedPtr<FColorTheme> ColorTheme;

	/** Event called when a color block is clicked */
	FOnLinearColorValueChanged OnSelectColor;

	/** Event called to retrieve the current color from the color picker */
	FOnGetActiveColor OnGetActiveColor;

	/** Callback to pass to the Color Theme. Holds a handle to this bar's Refresh method */
	FSimpleDelegate RefreshCallback;

	/** Handle to the registered RefreshCallback delegate */
	FDelegateHandle RefreshCallbackHandle;

	/** Whether to use display sRGB color */
	TAttribute<bool> bUseSRGB;

	/** Whether or not the color uses Alpha or not */
	TAttribute<bool> bUseAlpha;

	/** Whether or not the delete button is currently visible */
	bool bShowDeleteButton = false;

	/** Placeholder widget to show where a dropped color block will be added to the color theme */
	TSharedPtr<SBorder> DragShadow;

	/** Overlay widget that shows either the add button or the delete button */
	TSharedPtr<SOverlay> AddDeleteOverlay;

	/** Custom combo button widget featuring a menu with the list of color themes and actions related to those themes */
	TSharedPtr<SColorThemesViewer> ThemesViewer;

	/** Index in the panel indicating where the drag shadow placeholder widget should be drawn */
	TOptional<int32> PlaceholderIndex;
};


/**
 * This operation is a color which can be dragged and dropped between widgets.
 * Represents a SThemeColorBlock that is dragged around, and can be dropped into a color trash.
 */
class FColorDragDrop : public FDragDropOperation
{
public:

	DRAG_DROP_OPERATOR_TYPE(FColorDragDrop, FDragDropOperation)

	FColorDragDrop(FLinearColor InColor, bool bInUseSRGB, bool bInUseAlpha, FSimpleDelegate InTrashShowCallback, FSimpleDelegate InTrashHideCallback);

	// Begin FDragDropOperation overrides
	virtual void OnDrop( bool bDropWasHandled, const FPointerEvent& MouseEvent ) override;
	virtual void OnDragged( const class FDragDropEvent& DragDropEvent ) override;
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	// End FDragDropOperation overrides

	/** Makes the decorator window fully opaque to visually indicate that the color will be added when dropped */
	void MarkForAdd();

	/** Makes the decorator window slightly transparent to visually indicate that the color will be deleted when dropped */
	void MarkForDelete();

	/**
	 * Makes a new FColorDragDrop to hold on to
	 * @param InColor			The color to be dragged and dropped
	 * @param bSRGB				Whether the color is sRGB
	 * @param bUseAlpha			Whether the colors alpha is important
	 * @param TrashShowCallback	Called when this operation is created
	 * @param TrashHideCallback Called when this operation is dropped
	 * @param Origin			The SThemeColorBlockBar that this operation is from
	 * @param OriginPosition	The position in it's origin that it is from
	 */
	static TSharedRef<FColorDragDrop> New(FLinearColor InColor, bool bSRGB, bool bUseAlpha,
		FSimpleDelegate TrashShowCallback = FSimpleDelegate(), FSimpleDelegate TrashHideCallback = FSimpleDelegate(),
		TSharedPtr<SThemeColorBlocksBar> Origin = TSharedPtr<SThemeColorBlocksBar>(), int32 OriginPosition = 0);

	/** The color currently held onto by this drag drop operation */
	FLinearColor Color;

	/** Whether or not the color uses sRGB */
	bool bUseSRGB;

	/** Whether or not the color uses Alpha */
	bool bUseAlpha;

	UE_DEPRECATED(5.6, "OriginBar is deprecated. This operation no longer needs direct access to the SThemeColorBlocksBar widget.")
	TWeakPtr<SThemeColorBlocksBar> OriginBar;

	UE_DEPRECATED(5.6, "OriginBarPosition is deprecated. This operation no longer needs to know its position in the SThemeColorBlocksBar widget.")
	int32 OriginBarPosition;

	/** Callback to show the delete button of the SThemeColorBlocksBar when this is dropped */
	FSimpleDelegate ShowTrash;

	/** Callback to hide the delete button of the SThemeColorBlocksBar when this is dropped */
	FSimpleDelegate HideTrash;

	UE_DEPRECATED(5.6, "bSetForDeletion is deprecated. This operation no longer deletes any color blocks from the SThemeColorBlocksBar widget.")
	bool bSetForDeletion;

	UE_DEPRECATED(5.6, "BlockSize is deprecated. This operation now uses the size of the cursor decorator window internally.")
	FVector2D BlockSize;
};
