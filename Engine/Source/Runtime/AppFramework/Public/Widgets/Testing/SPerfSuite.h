// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if !UE_BUILD_SHIPPING

APPFRAMEWORK_API void SummonPerfTestSuite();

#endif // #if !UE_BUILD_SHIPPING
