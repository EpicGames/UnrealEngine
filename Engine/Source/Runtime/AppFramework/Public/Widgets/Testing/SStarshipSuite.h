// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"

#if !UE_BUILD_SHIPPING

extern APPFRAMEWORK_API const FLazyName StarshipGalleryTabName;
APPFRAMEWORK_API void RegisterStarshipGalleryTabSpawner();
APPFRAMEWORK_API void UnregisterStarshipGalleryTabSpawner();
APPFRAMEWORK_API void RestoreStarshipSuite();

#endif // #if !UE_BUILD_SHIPPING
