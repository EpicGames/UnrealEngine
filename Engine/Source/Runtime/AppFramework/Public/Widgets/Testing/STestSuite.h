// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if !UE_BUILD_SHIPPING

APPFRAMEWORK_API void RestoreSlateTestSuite();
APPFRAMEWORK_API void MakeSplitterTest();

#endif // #if !UE_BUILD_SHIPPING
