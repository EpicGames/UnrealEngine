// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Types/SlateStructs.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SHyperlink.h"

#define LOCTEXT_NAMESPACE "SMultipleOptionTable"

/**
 * Implements a row widget for an option list.
 */
template<typename OptionType>
class SOptionTableRow
	: public SMultiColumnTableRow< TSharedPtr<OptionType> >
{
public:
	typedef typename TSlateDelegates< OptionType >::FOnGenerateWidget FOnGenerateWidget;
	typedef typename STableRow< TSharedPtr<OptionType> >::FArguments FOptionTableRowArgs;

	SLATE_BEGIN_ARGS(SOptionTableRow) { }
		SLATE_ARGUMENT(TSharedPtr<STableViewBase>, OwnerTableView)
		SLATE_ARGUMENT(OptionType, Option)
		SLATE_EVENT(FOnCheckStateChanged, OnCheckStateChanged)
		SLATE_ATTRIBUTE(ECheckBoxState, IsChecked)
		SLATE_EVENT(FOnGenerateWidget, OnGenerateWidget)
	SLATE_END_ARGS()

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	void Construct( const FArguments& InArgs)
	{
		Option = InArgs._Option;
		OnCheckStateChanged = InArgs._OnCheckStateChanged;
		IsChecked = InArgs._IsChecked;
		OnGenerateWidget= InArgs._OnGenerateWidget;

		SMultiColumnTableRow< TSharedPtr<OptionType> >::Construct(FOptionTableRowArgs(), InArgs._OwnerTableView.ToSharedRef());
	}

	/**
	 * Generates the widget for the specified column.
	 *
	 * @param ColumnName The name of the column to generate the widget for.
	 *
	 * @return The widget.
	 */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		if (ColumnName == "IsSelected")
		{
			return SAssignNew(CheckBox, SCheckBox)
				.IsChecked(this, &SOptionTableRow::HandleCheckBoxIsChecked)
				.OnCheckStateChanged(this, &SOptionTableRow::HandleCheckBoxCheckStateChanged)
				[
					OnGenerateWidget.Execute(Option)
				];
		}

		return SNullWidget::NullWidget;
	}

private:
	// Callback for changing the checked state of the check box.
	void HandleCheckBoxCheckStateChanged( ECheckBoxState NewState )
	{
		OnCheckStateChanged.Execute(NewState);
	}

	// Callback for determining the checked state of the check box.
	ECheckBoxState HandleCheckBoxIsChecked( ) const
	{
		return IsChecked.Get();
	}

public:
	TSharedPtr<SCheckBox> CheckBox;

private:
	OptionType Option;
	FOnCheckStateChanged OnCheckStateChanged;
	TAttribute<ECheckBoxState> IsChecked;
	FOnGenerateWidget OnGenerateWidget;
};

template<typename OptionType>
class SMultipleOptionTable : public SCompoundWidget
{
public:
	typedef typename TSlateDelegates< OptionType >::FOnGenerateWidget FOnGenerateOptionWidget;

	// This callback will be used before selecting or deselecting all of the options. May be of use for optimization.
	DECLARE_DELEGATE(FOnPreBatchSelect);

	// This callback will be used after selecting or deselecting all of the options. May be of use for optimization.
	DECLARE_DELEGATE(FOnPostBatchSelect);

	DECLARE_DELEGATE_TwoParams(FOnOptionSelectionChanged, bool, OptionType);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsOptionSelected, OptionType);

	SLATE_BEGIN_ARGS( SMultipleOptionTable<OptionType> ) {}
		SLATE_EVENT(FOnPreBatchSelect, OnPreBatchSelect)
		SLATE_EVENT(FOnPostBatchSelect, OnPostBatchSelect)
		SLATE_EVENT(FOnGenerateOptionWidget, OnGenerateOptionWidget)
		SLATE_EVENT(FOnOptionSelectionChanged, OnOptionSelectionChanged)
		SLATE_EVENT(FIsOptionSelected, IsOptionSelected)
		SLATE_ATTRIBUTE(FOptionalSize, ListHeight)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TArray< OptionType >* const InOptions);

	// Refreshes the list view. Useful if the options array is modified.
	void RequestTableRefresh();

private:
	// Generates the special row widget for this list view widget.
	TSharedRef<class ITableRow> HandleOptionListViewGenerateRow(OptionType Option, const TSharedRef< class STableViewBase >& OwnerTableView);

	// Calls the provided user callback to generate a widget to display the specified option.
	TSharedRef<SWidget> GenerateWidgetForOption(OptionType Option);

	// Calls the provided user callback to notify them that an option has been checked or unchecked.
	void HandleCheckBoxCheckStateChanged(ECheckBoxState NewState, OptionType Option) const;

	// Calls the provided user callback to be notified that an option is selected or unselected in some way.
	ECheckBoxState HandleCheckBoxIsChecked(OptionType Option) const;

	// Selects all options.
	void HandleAllHyperlinkNavigate();

	// Deselects all options.
	void HandleNoneHyperlinkNavigate();

	// Collapses hyperlinks for selecting/deselecting all options based on whether there are multiple options.
	EVisibility HandleHyperlinkVisibility() const;

private:
	const TArray< OptionType >* Options;
	FOnPreBatchSelect OnPreBatchSelect;
	FOnPostBatchSelect OnPostBatchSelect;
	FOnGenerateOptionWidget OnGenerateOptionWidget;
	FOnOptionSelectionChanged OnOptionSelectionChanged;
	FIsOptionSelected IsOptionSelected;
	TAttribute<FOptionalSize> ListHeight;

	TSharedPtr< SListView< OptionType > > OptionListView;
};

template<typename OptionType>
void SMultipleOptionTable<OptionType>::Construct(const FArguments& InArgs, const TArray< OptionType >* const InOptions)
{
	Options = InOptions;
	OnPreBatchSelect = InArgs._OnPreBatchSelect;
	OnPostBatchSelect = InArgs._OnPostBatchSelect;
	OnGenerateOptionWidget = InArgs._OnGenerateOptionWidget;
	OnOptionSelectionChanged = InArgs._OnOptionSelectionChanged;
	IsOptionSelected = InArgs._IsOptionSelected;
	ListHeight = InArgs._ListHeight;

	ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.FillHeight(1.0)
			.Padding(0.0f, 2.0f, 0.0f, 0.0f)
			[
				SNew(SBox)
				.HeightOverride(ListHeight)
				[
					// Options menu
					SAssignNew(OptionListView, SListView<OptionType>)
					.HeaderRow
					(
					SNew(SHeaderRow)
					.Visibility(EVisibility::Collapsed)

					+ SHeaderRow::Column("IsSelected")
					.DefaultLabel(LOCTEXT("OptionListIsSelectedColumnHeader", "IsSelected"))
					.FillWidth(1.0f)
					)
					.ListItemsSource(Options)
					.OnGenerateRow(this, &SMultipleOptionTable::HandleOptionListViewGenerateRow)
					.SelectionMode(ESelectionMode::None)
				]
			]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 6.0f, 0.0f, 4.0f)
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
				]

			+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.HAlign(HAlign_Right)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("SelectLabel", "Select:"))
					]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(8.0f, 0.0f)
						[
							// Select all options hyper link
							SNew(SHyperlink)
							.OnNavigate(this, &SMultipleOptionTable::HandleAllHyperlinkNavigate)
							.Text(LOCTEXT("AllHyperlinkLabel", "All"))
							.ToolTipText(LOCTEXT("AllHyperlinkToolTip", "Select all options."))
							.Visibility(this, &SMultipleOptionTable::HandleHyperlinkVisibility)									
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							// Deselect all options hyper link
							SNew(SHyperlink)
							.OnNavigate(this, &SMultipleOptionTable::HandleNoneHyperlinkNavigate)
							.Text(LOCTEXT("NoneHyperlinkLabel", "None"))
							.ToolTipText(LOCTEXT("NoneHyperlinkToolTip", "Deselect all."))
							.Visibility(this, &SMultipleOptionTable::HandleHyperlinkVisibility)									
						]
				]
		];
}

template<typename OptionType>
void SMultipleOptionTable<OptionType>::RequestTableRefresh()
{
	OptionListView->RequestListRefresh();
}

template<typename OptionType>
TSharedRef<class ITableRow> SMultipleOptionTable<OptionType>::HandleOptionListViewGenerateRow(OptionType Option, const TSharedRef< class STableViewBase >& OwnerTableView)
{
	return SNew(SOptionTableRow<OptionType>)
		.Option(Option)
		.OwnerTableView(OwnerTableView)
		.OnCheckStateChanged(this, &SMultipleOptionTable::HandleCheckBoxCheckStateChanged, Option)
		.IsChecked(this, &SMultipleOptionTable::HandleCheckBoxIsChecked, Option)
		.OnGenerateWidget(this, &SMultipleOptionTable::GenerateWidgetForOption);
}

template<typename OptionType>
TSharedRef<SWidget> SMultipleOptionTable<OptionType>::GenerateWidgetForOption(OptionType Option)
{
	return OnGenerateOptionWidget.Execute(Option);
}

template<typename OptionType>
void SMultipleOptionTable<OptionType>::HandleCheckBoxCheckStateChanged(ECheckBoxState NewState, OptionType Option) const
{
	OnOptionSelectionChanged.Execute(NewState == ECheckBoxState::Checked, Option);
}

template<typename OptionType>
ECheckBoxState SMultipleOptionTable<OptionType>::HandleCheckBoxIsChecked(OptionType Option) const
{
	return IsOptionSelected.Execute(Option) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

template<typename OptionType>
void SMultipleOptionTable<OptionType>::HandleAllHyperlinkNavigate()
{
	OnPreBatchSelect.ExecuteIfBound();
	for (const OptionType& Option : *Options)
	{
		HandleCheckBoxCheckStateChanged(ECheckBoxState::Checked, Option);
	}
	OnPostBatchSelect.ExecuteIfBound();
}

template<typename OptionType>
void SMultipleOptionTable<OptionType>::HandleNoneHyperlinkNavigate()
{
	OnPreBatchSelect.ExecuteIfBound();
	for (const OptionType& Option : *Options)
	{
		HandleCheckBoxCheckStateChanged(ECheckBoxState::Unchecked, Option);
	}
	OnPostBatchSelect.ExecuteIfBound();
}

template<typename OptionType>
EVisibility SMultipleOptionTable<OptionType>::HandleHyperlinkVisibility() const
{
	if (Options->Num() > 1)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
