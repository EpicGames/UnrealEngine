// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Containers/IndirectArray.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Framework/SlateDelegates.h"
#include "Styling/AppStyle.h"

/**
 * Delegate type for getting the next page to show.
 *
 * The parameter is the current page index.
 * The return value should be the next page index.
 */
DECLARE_DELEGATE_RetVal_OneParam(int32, FOnGetPageIndex, int32)

/**
 * Implements a wizard widget.
 */
class SWizard
	: public SCompoundWidget
{
public:

	/**
	 * Implements a wizard page.
	 */
	class FWizardPage
	{
	public:

		SLATE_BEGIN_ARGS(FWizardPage)
			: _ButtonContent()
			, _CanShow(true)
			, _PageContent()
		{ }

			/** Holds the content of the button to be shown in the wizard's page selector. */
			SLATE_NAMED_SLOT(FArguments, ButtonContent)
			
			/** Holds the name of the page to be displayed in the breadcrumb trail. */
			SLATE_ATTRIBUTE(FText, Name)

			/** Holds a flag indicating whether this page can be activated (default = true). */
			SLATE_ATTRIBUTE(bool, CanShow)

			/** Exposes a delegate to be invoked when this page is being activated. */
			SLATE_EVENT(FSimpleDelegate, OnEnter)

			/** Exposes a delegate to be invoked when this page is being deactivated. */
			SLATE_EVENT(FSimpleDelegate, OnLeave)

			/** Holds the page content. */
			SLATE_DEFAULT_SLOT(FArguments, PageContent)

		SLATE_END_ARGS()

	public:

		/**
		 * Creates and initializes a new instance.
		 *
		 * @param InArgs The arguments for the page.
		 */
		FWizardPage( const FArguments& InArgs )
			: ButtonContent(InArgs._ButtonContent)
			, Showable(InArgs._CanShow)
			, OnEnterDelegate(InArgs._OnEnter)
			, OnLeaveDelegate(InArgs._OnLeave)
			, PageContent(InArgs._PageContent)
			, Name(InArgs._Name)
		{ }

	public:

		/**
		 * Checks whether the page can be shown.
		 *
		 * @return true if the page can be shown, false otherwise.
		 */
		bool CanShow() const
		{
			return Showable.Get();
		}

		/** Gets the display name of the page. */
		FText GetName() const
		{
			return Name.Get();
		}

		/**
		 * Gets the button content.
		 *
		 * @return The button content widget.
		 */
		const TSharedRef<SWidget>& GetButtonContent() const
		{
			return ButtonContent.Widget;
		}

		/**
		 * Gets the page content.
		 *
		 * @return The page contents.
		 */
		const TSharedRef<SWidget>& GetPageContent() const
		{
			return PageContent.Widget;
		}

		/**
		 * Gets a delegate to be invoked when this page is being entered.
		 *
		 * @return The delegate.
		 */
		FSimpleDelegate& OnEnter()
		{
			return OnEnterDelegate;
		}

		/**
		 * Gets a delegate to be invoked when this page is being left.
		 *
		 * @return The delegate.
		 */
		FSimpleDelegate& OnLeave()
		{
			return OnLeaveDelegate;
		}

	private:

		// Holds the button content.
		TAlwaysValidWidget ButtonContent;

		// Holds a flag indicating whether this page can be activated.
		TAttribute<bool> Showable;

		// Holds a delegate to be invoked when the page is activated.
		FSimpleDelegate OnEnterDelegate;

		// Holds a delegate to be invoked when the page is deactivated.
		FSimpleDelegate OnLeaveDelegate;

		// Holds the page content.
		TAlwaysValidWidget PageContent;

		// Holds the name of the page.
		TAttribute<FText> Name;
	};

public:

	SLATE_BEGIN_ARGS(SWizard)
		: _ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button"))
		, _CancelButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Button"))
		, _FinishButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("PrimaryButton"))
		, _ButtonTextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("DialogButtonText"))
		, _CanFinish(true)
		, _FinishButtonText(NSLOCTEXT("SWizard", "DefaultFinishButtonText", "Finish"))
		, _FinishButtonToolTip(NSLOCTEXT("SWizard", "DefaultFinishButtonTooltip", "Finish the wizard"))
		, _InitialPageIndex(0)
		, _DesiredSize(FVector2D(0, 0))
		, _ShowPageList(true)
		, _ShowCancelButton(true)
		, _ShowBreadcrumbs(false)
		, _BreadcrumbButtonStyle(&FCoreStyle::Get().GetWidgetStyle< FButtonStyle >("BreadcrumbButton"))
		, _BreadcrumbTextStyle(&FCoreStyle::Get().GetWidgetStyle< FTextBlockStyle >("NormalText"))
		, _ShowHistoryButtons(true)
		, _ShowPageTitle(false)
		, _PageTitleTextStyle(&FCoreStyle::Get().GetWidgetStyle< FTextBlockStyle >("Wizard.PageTitle"))
		, _PageFooter()
	{ }

	SLATE_SUPPORTS_SLOT_WITH_ARGS(FWizardPage)

		/** The button style used by next and back. */
		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)

		/** The button style used by the cancel button. */
		SLATE_STYLE_ARGUMENT(FButtonStyle, CancelButtonStyle)

		/** The button style used by the finish button. */
		SLATE_STYLE_ARGUMENT(FButtonStyle, FinishButtonStyle)

		/** The text style used by the buttons. */
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, ButtonTextStyle)

		/** Holds a flag indicating whether the 'Finish' button is enabled. */
		SLATE_ATTRIBUTE(bool, CanFinish)

		/** Holds the label text of the wizard's 'Finish' button. */
		SLATE_ATTRIBUTE(FText, FinishButtonText)

		/** Holds the tool tip text of the wizard's 'Finish' button. */
		SLATE_ATTRIBUTE(FText, FinishButtonToolTip)

		/** Holds the index of the initial page to be displayed (0 = default). */
		SLATE_ATTRIBUTE(int32, InitialPageIndex)

		/** Holds the desired size. */
		SLATE_ATTRIBUTE(FVector2D, DesiredSize)

		/** Exposes a delegate to be invoked when the wizard is canceled. */
		SLATE_EVENT(FSimpleDelegate, OnCanceled)

		/** Exposes a delegate to be invoked when the wizard is finished. */
		SLATE_EVENT(FSimpleDelegate, OnFinished)

		/** Exposes a delegate to be invoked when the wizard's 'Back' button is clicked. This button is only visible if this delegate is set and only on the first page. */
		SLATE_EVENT(FOnClicked, OnFirstPageBackClicked)

		/**
		 * Exposes a delegate to be invoked to control whether the wizard's 'Next' button should be active, and which page should be next if it is clicked.
		 * Significant work should not be done in this delegate as it will be polled regularly by the button.
		 * Passes in the current page index and expects the next page index to be returned.
		 */
		SLATE_EVENT(FOnGetPageIndex, OnGetNextPageIndex)

		/**
		 * Exposes a delegate to be invoked when the wizard tries to move to new next page e.g. when 'Next' is pressed.
		 * Allows the wizard to redirect to another page if processing of the current step fails in some way.
		 * Passes in the page index the wizard is trying to move to and expects the actual next page index to be returned.
		 * This process will not be repeated; the replacement page must be able to be moved to without failure.
		 */
		SLATE_EVENT(FOnGetPageIndex, OnTryToMoveToPage)

		/** Holds a flag indicating whether the page list should be shown (default = true). */
		SLATE_ARGUMENT(bool, ShowPageList)

		/** Holds a flag indicating whether the cancel button should be shown (default = true). */
		SLATE_ARGUMENT(bool, ShowCancelButton)

		/** Holds a flag indicating whether a breadcrumb trail should be shown (default = false) */
		SLATE_ARGUMENT(bool, ShowBreadcrumbs)

		/** The button style to use for each breadcrumb if ShowBreadcrumbs is set to true. */
		SLATE_STYLE_ARGUMENT(FButtonStyle, BreadcrumbButtonStyle);

		/** The text style to use for breadcrumbs if ShowBreadcrumbs is set to true. */
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, BreadcrumbTextStyle);

		/** Holds a flag indicating whether the previous and next buttons can be shown (default = true) */
		SLATE_ARGUMENT(bool, ShowHistoryButtons)

		/** Holds a flag indicating whether a page title should be shown for each page (default = false) */
		SLATE_ARGUMENT(bool, ShowPageTitle)

		/** The text style to use for showing the page title if ShowPageTitle is set to true. */
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, PageTitleTextStyle);

		/** Holds an optional widget containing the contents to place above the buttons for all wizard pages */
		SLATE_NAMED_SLOT(FArguments, PageFooter)

	SLATE_END_ARGS()

public:

	/**
	 * Checks whether the page with the specified index can be shown.
	 *
	 * @param PageIndex The index of the page to check.
	 * @return true if the page can be shown, false otherwise.
	 */
	APPFRAMEWORK_API bool CanShowPage( int32 PageIndex ) const;

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	APPFRAMEWORK_API void Construct( const FArguments& InArgs );

	/**
	 * Gets the number of pages that this wizard contains.
	 *
	 * @return The number of pages.
	 */
	APPFRAMEWORK_API int32 GetNumPages() const;

	/**
	 * Get the index of the current page.
	 * @return The index of the current page.
	 */
	APPFRAMEWORK_API int32 GetCurrentPageIndex() const;

	/**
	 * Gets the index of the specified wizard page widget.
	 *
	 * @param PageWidget The page widget to get the index for.
	 * @return The index of the page, or INDEX_NONE if not found.
	 */
	APPFRAMEWORK_API int32 GetPageIndex( const TSharedPtr<SWidget>& PageWidget ) const;

	/**
	 * Attempts to show the page with the specified index.
	 *
	 * @param PageIndex The index of the page to show.
	 */
	APPFRAMEWORK_API void ShowPage( int32 PageIndex );

	/** Shows the page with the specified index, but preserves page history. */
	APPFRAMEWORK_API void AdvanceToPage( int32 PageIndex );

	/** Get the widget to focus on activation */
	APPFRAMEWORK_API TSharedPtr<SWidget> GetWidgetToFocusOnActivate();

public:

	// SCompoundWidget interface

	APPFRAMEWORK_API virtual FVector2D ComputeDesiredSize(float) const override;

public:

	/**
	 * Returns a new slot for a page.
	 *
	 * @param PageName The name of the page.
	 * @return A new slot.
	 */
	static FWizardPage::FArguments Page()
	{
		FWizardPage::FArguments Args;
		return Args;
	}

private:

	// Callback for clicking the 'Cancel' button.
	APPFRAMEWORK_API FReply HandleCancelButtonClicked();

	// Callback for clicking the 'Launch' button.
	APPFRAMEWORK_API FReply HandleFinishButtonClicked();

	// Callback for clicking the 'Next' button.
	APPFRAMEWORK_API FReply HandleNextButtonClicked();

	// Callback for getting the enabled state of the 'Next' button.
	APPFRAMEWORK_API bool HandleNextButtonIsEnabled() const;

	// Callback for getting the visibility of the 'Next' button.
	APPFRAMEWORK_API EVisibility HandleNextButtonVisibility() const;

	// Callback for getting the checked state of a page button.
	APPFRAMEWORK_API void HandlePageButtonCheckStateChanged( ECheckBoxState NewState, int32 PageIndex );

	// Callback for clicking a page button.
	APPFRAMEWORK_API ECheckBoxState HandlePageButtonIsChecked( int32 PageIndex ) const;

	// Callback for getting the enabled state of a page button.
	APPFRAMEWORK_API bool HandlePageButtonIsEnabled( int32 PageIndex ) const;

	// Callback for clicking the 'Previous' button.
	APPFRAMEWORK_API FReply HandlePrevButtonClicked();

	// Callback for getting the enabled state of the 'Previous' button.
	APPFRAMEWORK_API bool HandlePrevButtonIsEnabled() const;

	// Callback for getting the visibility of the 'Previous' button.
	APPFRAMEWORK_API EVisibility HandlePrevButtonVisibility() const;

	// Callback for when a breadcrumb is clicked.
	APPFRAMEWORK_API void HandleBreadcrumbClicked(const int32& PageIndex);

	// Get the next page index to display.
	APPFRAMEWORK_API int32 GetNextPageIndex() const;

	// Get the previous page index to display.
	APPFRAMEWORK_API int32 GetPrevPageIndex() const;

	// Get the page title to display for the current page.
	APPFRAMEWORK_API FText HandleGetPageTitle() const;

private:

	// Holds the wizard's desired size.
	FVector2D DesiredSize;

	// Holds the collection of wizard pages.
	TIndirectArray<FWizardPage> Pages;

	// Holds the widget switcher.
	TSharedPtr<SWidgetSwitcher> WidgetSwitcher;

	// Holds the breadcrumb trail.
	TSharedPtr<SBreadcrumbTrail<int32>> BreadcrumbTrail;

	// Whether to show the "Previous" and "Next" buttons
	bool bShowHistoryButtons;

	TSharedPtr<SWidget> FinishButton;

private:

	// Holds a delegate to be invoked when the 'Cancel' button has been clicked.
	FSimpleDelegate OnCanceled;

	// Holds a delegate to be invoked when the 'Finish' button has been clicked.
	FSimpleDelegate OnFinished;

	// Holds a delegate to be invoked when the 'Next' button has been clicked.
	FOnClicked OnNextClicked;

	// Holds a delegate to be invoked when the 'Previous' button has been clicked.
	FOnClicked OnPrevClicked;

	// Holds a delegate to be invoked when the 'Previous' button has been clicked on the first page.
	FOnClicked OnFirstPageBackClicked;

	// Exposes a delegate to be invoked to control whether the 'Next' button is enabled, and where to attempt to move to if it is clicked.
	FOnGetPageIndex OnGetNextPageIndex;

	// Exposes a delegate to be invoked when the wizard's 'Next' button is clicked to allow redirection to another page if processing fails
	FOnGetPageIndex OnTryToMoveToPage;
};
